{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Accumulation Depth of Field",
	"Description": "Thin-lens aperture-sampled depth of field for production rendering",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "Epic Games Inc.",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AccumulationDOF",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Mac", "Win64", "Linux" ]
		},
		{
			"Name": "AccumulationDOFEditor",
			"Type": "Editor"
		}
	],
	"Plugins": [
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		}
	]
}
