// Copyright Epic Games, Inc. All Rights Reserved.

//
// Aperture sample accumulation
//

// Accumulates a single aperture sample into the running DOF buffer with weighted blending.
// Each sample represents the scene rendered from one point on the camera aperture disk.
//
// The shader computes per-sample weight from multiple sources, matching other Engine implementations (such as deferred and PT when applicable):
//
// - Anamorphic squeeze.
// - Aperture blades.
// - Petzval bokeh.
// - Cat's eye due to occlusion by the lens barrel.
// - Bokeh texture. Alpha or luminance channel controls transmission. RGB optionally tints.
// - Spectral weight parameter (used e.g. when rendering axial chromatic aberration bands).
//
// Output: Additive accumulation of weighted sample RGB, plus total weight in alpha for normalization.
//

#include "/Engine/Public/Platform.ush"

Texture2D PrevAccumTexture;
SamplerState PrevAccumSampler;

Texture2D SampleTexture;
SamplerState SampleSampler;

Texture2D BokehTexture;
SamplerState BokehSampler;

float2 ApertureOffsetCm;
float ApertureRadiusCm;
int BokehWeightChannel; // 0 = Alpha, 1 = Luminance
int BokehUseTint;
float BokehTintStrength;
int BokehEnabled;

// Anamorphic squeeze factor for bokeh
float SqueezeFactor;

// Spectral weight for axial chromatic aberration (1,1,1 : none)
float3 SpectralWeight;

// Soft falloff at aperture edge (0: hard edge, 1: very soft)
float BokehEdgeSoftness;

// Petzval field curvature parameters. See PathTracingCore.ush for usage reference.
float Petzval;
float PetzvalFalloffPower;
float2 PetzvalExclusionBoxExtents;
float PetzvalExclusionBoxRadius;

// Cat's eye bokeh from occlusion by the lens barrel
float BarrelRadiusCm;
float BarrelLengthCm;
float2 SensorHalfSizeCm;
float FocalLengthCm;
float FocusDistanceCm;

// Diaphragm blade parameters
int DiaphragmBladeCount;              // 4-16
float DiaphragmRotationRad;
int BokehShape;                       // 0:Circle, 1:StraightBlades, 2:RoundedBlades
float CocRadiusToCircumscribedRadius; // Scale factor for circumscribed radius
float CocRadiusToIncircleRadius;      // Scale factor for incircle radius
float DiaphragmBladeRadius;           // Blade arc radius
float DiaphragmBladeCenterOffset;     // Blade arc center offset

// AA reconstruction weight
float AntiAliasingWeight;

// Subpixel splatting 0-1
float2 SubpixelOffset;
float2 TexelSize;

// Coma aberration strength (Seidel W131)
float ComaStrength;

// Bokeh shape constants (matching DiaphragmDOF.h EBokehShape)
#define BOKEH_SHAPE_CIRCLE          0
#define BOKEH_SHAPE_STRAIGHT_BLADES 1
#define BOKEH_SHAPE_ROUNDED_BLADES  2

// Constants
#ifndef PI
#define PI 3.14159265358979323846
#endif

// Compute luminance from RGB using Rec. 709 coefficients.
// Based on CompositeColorKeyerCommon.ush
//
float LuminanceSRGB(float3 InColor)
{
	return dot(InColor, float3(0.2126390059, 0.7151686788, 0.0721923154));
}

// Compute scaling factor that needs to be applied to radial coordinate (R=1, PixelAngle)'s radius R,
// to have the point projected onto the bokeh's edge with the bokeh area remaining equal to PI.
//
// Based on DOFBokehLUT.usf ComputeCocRadiusToBokehEdgeFactor()
//
float ComputeCocRadiusToBokehEdgeFactor(float PixelAngle)
{
	if (BokehShape == BOKEH_SHAPE_CIRCLE)
	{
		return 1.0;
	}

	float EdgeId = floor(PixelAngle * rcp(2.0 * PI) * float(DiaphragmBladeCount));
	float EdgeAngle = (EdgeId + 0.5) * (2.0 * PI) * rcp(float(DiaphragmBladeCount));

	float Alpha = PixelAngle - EdgeAngle;

	if (BokehShape == BOKEH_SHAPE_STRAIGHT_BLADES)
	{
		return CocRadiusToIncircleRadius / cos(Alpha);
	}
	else // BOKEH_SHAPE_ROUNDED_BLADES
	{
		/**
		 * Resolve t:
		 *  (x + d) ^ 2 + y ^ 2 = r ^ 2
		 *  x = t cos(alpha)
		 *  y = t sin(alpha)
		 *
		 *  => t^2 + t * (2 * d * cos(alpha) + d^2 - r^2 = 0
		 */
		float b = 2.0 * DiaphragmBladeCenterOffset * cos(Alpha);
		float c = DiaphragmBladeCenterOffset * DiaphragmBladeCenterOffset
				- DiaphragmBladeRadius * DiaphragmBladeRadius;
		float Delta = b * b - 4.0 * c;

		/**
		 * Only want positive solution that we know exists because:
		 *  a > 0, b > 0 and c < 0 => Delta > b^2 => t > 0.
		 */
		return (-b + sqrt(Delta)) * 0.5;
	}
}

// Compute blade-shaped mask with edge softness
//
// @param DiskPos normalized disk position [-1,1] after Petzval and squeeze distortion
// @return 1.0 inside blade boundary (with soft falloff), 0.0 outside
//
float ComputeBladeMask(float2 DiskPos, float EdgeSoftness)
{
	if (BokehShape == BOKEH_SHAPE_CIRCLE)
	{
		// Fall back to circular edge softness for circle shape

		float Radius = length(DiskPos);

		if (EdgeSoftness <= 0.0)
		{
			return Radius <= 1.0 ? 1.0 : 0.0;
		}

		float EdgeStart = 1.0 - EdgeSoftness;
		float EdgeFalloff = 1.0 - saturate((Radius - EdgeStart) / max(EdgeSoftness, 0.001));
		return smoothstep(0.0, 1.0, EdgeFalloff);
	}

	float Radius = length(DiskPos);

	// Handle center case to avoid atan2 issues
	if (Radius < 0.0001)
	{
		return 1.0;
	}

	// Compute angle in blade space (with rotation offset)
	// Adding PI to match engine convention (blade 0 points in -X direction initially)
	float Angle = atan2(DiskPos.y, DiskPos.x) + DiaphragmRotationRad + PI;

	// Normalize angle to [0, 2*PI)
	Angle = fmod(Angle + 2.0 * PI, 2.0 * PI);

	// Get the edge factor for this angle (distance from center to blade edge)
	float EdgeFactor = ComputeCocRadiusToBokehEdgeFactor(Angle);

	// Normalize radius by edge factor - this gives us distance relative to blade boundary
	// NormalizedRadius = 1.0 at blade edge, < 1.0 inside, > 1.0 outside
	float NormalizedRadius = Radius / max(EdgeFactor, 0.0001);

	// Apply edge softness at blade boundary
	if (EdgeSoftness <= 0.0)
	{
		return NormalizedRadius <= 1.0 ? 1.0 : 0.0;
	}

	// Soft falloff: starts at (1 - EdgeSoftness), reaches 0 at 1.0
	float EdgeStart = 1.0 - EdgeSoftness;
	float EdgeFalloff = 1.0 - saturate((NormalizedRadius - EdgeStart) / max(EdgeSoftness, 0.001));
	return smoothstep(0.0, 1.0, EdgeFalloff);
}

// Apply Petzval field curvature distortion to bokeh shape
//
// This is the inverse of PathTracingCore.ush ApplyPetzval() because:
// - Path tracer compresses lens sample positions, i.e. narrower bokeh in that axis
// - We sample a bokeh texture, so we need to expand the UV to get narrower result
//
// @param BokehCenterUV Screen UV [0,1] of the pixel being shaded
// @param SamplePos Normalized disk position [-1,1] of the aperture sample
//
// @return Distorted disk position [-1,1] stretched radially or tangentially based on screen position
float2 ApplyPetzvalDistortion(float2 BokehCenterUV, float2 SamplePos)
{
	if (Petzval != 0)
	{
		// Convert UV to NDC [-1,1]
		float2 BokehCenter = BokehCenterUV * 2.0 - 1.0;

		// Flip Y to match path tracer convention (Y-up in NDC)
		BokehCenter.y *= -1.0;

		// Compute distance from exclusion box with rounded corners
		const float Radius = PetzvalExclusionBoxRadius * min(PetzvalExclusionBoxExtents.x, PetzvalExclusionBoxExtents.y);
		const float2 BoxToBokeh = sign(BokehCenter) * max(abs(BokehCenter) - PetzvalExclusionBoxExtents + Radius, 0.0);
		float BokehDistance = max(0.0, length(BoxToBokeh));
		const float2 Normal = (BokehDistance > 0) ? (BoxToBokeh / BokehDistance) : float2(0, 0);
		BokehDistance = max(0.0, BokehDistance - Radius);

		if (BokehDistance > 0)
		{
			// Tangent is perpendicular to normal (radial direction)
			const float2 Tangent = float2(Normal.y, -Normal.x);

			// Compute stretch factor based on distance from exclusion box
			const float Intensity = pow(BokehDistance, PetzvalFalloffPower);
			
			// Invers of the PT implementation: expand instead of compress
			// PT uses: Stretch = rcp(1 + abs(Petzval) * Intensity) which is < 1
			// We use the reciprocal to expand: Stretch = 1 + abs(Petzval) * Intensity which is > 1
			const float Stretch = 1.0 + abs(Petzval) * Intensity;

			// Transform to sagittal/tangential space
			float2x2 ToSaggitalSpace = { Normal.x, Normal.y, Tangent.x, Tangent.y };

			float2 SagittalPos = mul(SamplePos, ToSaggitalSpace);

			// Apply stretch in sagittal or tangential direction based on sign
			bool bMask = Petzval > 0;
			SagittalPos.x *=  bMask ? Stretch : 1.0; // Sagittal (radial)
			SagittalPos.y *= !bMask ? Stretch : 1.0; // Tangential

			// Transform back to screen space
			return mul(SagittalPos, transpose(ToSaggitalSpace));
		}
	}
	return SamplePos;
}

// Compute barrel occlusion for cat's eye bokeh effect with soft edges
//
// When EdgeSoftness > 0, applies a smooth falloff at the barrel boundary based on
// how close the ray passes to the barrel wall, creating smoother cat's eye edges.
//
// @param BokehCenterUV    - Screen UV [0,1] of the pixel being shaded
// @param ApertureOffsetCm - Aperture sample offset from optical axis in cm
// @param ApertureRadiusCm - Aperture radius in cm (used for soft edge scaling)
// @param EdgeSoftness     - Softness factor for barrel boundary (0 = hard edge, 1 = very soft)
//
// @return 1.0 if not occluded, soft gradient 0..1 at edge, 0.0 if fully occluded
//
float ComputeBarrelOcclusion(float2 BokehCenterUV, float2 ApertureOffsetCm, float ApertureRadiusCm, float EdgeSoftness)
{
	if (BarrelLengthCm <= 0.0)
	{
		return 1.0;  // Barrel occlusion disabled
	}

	// Convert UV [0,1] to NDC [-1,1]
	float2 ScreenNDC = BokehCenterUV * 2.0 - 1.0;
	ScreenNDC.y *= -1.0;  // Flip Y to match camera convention

	// Compute focus plane point for this pixel, first projecting sensor edge to focus plane
	// then using that to scale the NDC coordinates to cm.
	float2 FocusPlaneHalfSizeCm = SensorHalfSizeCm * (FocusDistanceCm / FocalLengthCm);
	float3 PixelAtFocusPlaneCm = float3(ScreenNDC * FocusPlaneHalfSizeCm, FocusDistanceCm);

	// Ray from aperture sample position toward PixelAtFocusPlaneCm
	// Aperture sample is at (ApertureOffsetCm, 0) on the lens plane (z=0)
	float3 RayOrigin = float3(ApertureOffsetCm, 0.0);
	float3 RayDir = normalize(PixelAtFocusPlaneCm - RayOrigin);

	// Calculate Ray-cylinder intersection
	// -----------------------------------
	//
	// Barrel is a cylinder of radius BarrelRadiusCm centered on optical axis (0,0)
	// extending from z=0 to z=BarrelLengthCm
	//
	// Ray to intersect, where RayParam is the only unknown:
	//     Point(RayParam) = RayOrigin + RayParam * RayDir
	//
	// Cylinder equation (infinite cylinder along z-axis):
	//     x^2 + y^2 = R^2
	//
	// Substitute ray into cylinder:
	//     (RayOrigin.x + RayParam*Dir.x)^2 + (RayOrigin.y + RayParam*Dir.y)^2 = R^2
	//
	// Therefore:
	//
	//     (Dir.x^2 + Dir.y^2) * RayParam^2 + 2*(RayOrigin.x*Dir.x + RayOrigin.y*Dir.y) * RayParam + (RayOrigin.x^2 + RayOrigin.y^2 - R^2) = 0
	//                       A * RayParam^2 +                                         B * RayParam +                                     C = 0
	//
	// Then we use the quadratic formula to solve for RayParam and calculate the intersection.

	float A = dot(RayDir.xy, RayDir.xy);
	float B = 2.0 * dot(RayDir.xy, RayOrigin.xy);
	float C = dot(RayOrigin.xy, RayOrigin.xy) - BarrelRadiusCm * BarrelRadiusCm;
	float Det = B * B - 4.0 * A * C;

	if (Det < 0.0)
	{
		// No intersection with infinite cylinder
		// This means aperture sample is inside the barrel radius and ray doesn't exit
		return 1.0;
	}

	// Per quadratic formula: RayParam = (-B +/- sqrt(Det)) / (2*A)
	float SqrtDet = sqrt(Det);
	float Inv2A = rcp(2.0 * A);
	float RayParam1 = (-B + SqrtDet) * Inv2A;
	float RayParam2 = (-B - SqrtDet) * Inv2A;

	// At the intersection, the ray position is RayOrigin + RayParam * RayDir
	float Z1 = RayParam1 * RayDir.z;
	float Z2 = RayParam2 * RayDir.z;

	// Check if either intersection is within the barrel (0 < z < BarrelLengthCm)
	bool Hit1Valid = (RayParam1 > 0.0) && (Z1 > 0.0) && (Z1 < BarrelLengthCm);
	bool Hit2Valid = (RayParam2 > 0.0) && (Z2 > 0.0) && (Z2 < BarrelLengthCm);

	// Ray misses barrel
	if (!Hit1Valid && !Hit2Valid)
	{
		return 1.0;
	}
	
	// Soft Edge computation
	//
	
	if (EdgeSoftness <= 0.0)
	{
		return 0.0;
	}

	// Soft falloff based on how close the hit is to the barrel z-boundary
	float HitZ = Hit1Valid ? Z1 : Z2;
	float ZEdgeDist = min(HitZ, BarrelLengthCm - HitZ);

	float BarrelSoftZone = BarrelLengthCm * EdgeSoftness;
	float NormalizedZEdge = ZEdgeDist / max(BarrelSoftZone, 0.001);

	return 1.0 - smoothstep(0.0, 1.0, saturate(NormalizedZEdge));
}


// Compute coma aberration UV shift (Seidel W131)
//
// @param ScreenUV  Screen UV [0,1] of the pixel
// @param NormPupil Normalized aperture position [-1,1]
//
// @return UV offset to apply when sampling
//
float2 ComputeComaShift(float2 ScreenUV, float2 NormPupil)
{
	if (ComaStrength <= 0.0 || FocalLengthCm <= 0.0 || ApertureRadiusCm <= 0.0001)
	{
		return float2(0.0, 0.0);
	}

	// UV to sensor space
	float2 ScreenNDC = ScreenUV * 2.0 - 1.0;
	ScreenNDC.y *= -1.0;
	float2 FieldPosCm = ScreenNDC * SensorHalfSizeCm;
	float FieldHeightCm = length(FieldPosCm);
	float FieldNorm = FieldHeightCm / max(length(SensorHalfSizeCm), 0.0001);

	if (FieldNorm < 0.0001)
	{
		return float2(0.0, 0.0);
	}

	// Decompose pupil into tangential and sagittal components
	float2 FieldDir = FieldPosCm / FieldHeightCm;
	float2 SagDir = float2(-FieldDir.y, FieldDir.x);
	float PupilT = dot(NormPupil, FieldDir);
	float PupilS = dot(NormPupil, SagDir);

	// Seidel W131 ray aberration
	float ShiftT = FieldNorm * (PupilS * PupilS + 3.0 * PupilT * PupilT);
	float ShiftS = FieldNorm * 2.0 * PupilS * PupilT;
	float2 ShiftDir = ShiftT * FieldDir + ShiftS * SagDir;

	// Convert to UV offset
	return float2(-ShiftDir.x, ShiftDir.y) * ComaStrength / (2.0 * SensorHalfSizeCm);
}


void MainVS(
	in  uint   VertexId    : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV       : TEXCOORD0
)
{
	// Full-screen triangle
	OutUV = float2((VertexId << 1) & 2, VertexId & 2);
	OutPosition = float4(OutUV * float2(2, -2) + float2(-1, 1), 0, 1);
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2 UV         : TEXCOORD0,
	out float4 OutColor   : SV_Target0
)
{
	float4 PrevAccum = PrevAccumTexture.Sample(PrevAccumSampler, UV);

	// Map aperture offset to normalized disk coordinates [-1, 1]
	float2 NormDisk = ApertureOffsetCm / max(ApertureRadiusCm, 0.0001);

	// Bilinear subpixel splatting + coma aberration UV shift
	float2 GatherOffset = (SubpixelOffset - 0.5) * TexelSize;
	float2 ComaOffset = ComputeComaShift(UV, NormDisk);
	float4 Sample = SampleTexture.Sample(SampleSampler, UV + GatherOffset + ComaOffset);

	// Apply Petzval field curvature distortion based on screen position
	float2 DistortedDisk = ApplyPetzvalDistortion(UV, NormDisk);

	// Apply anamorphic squeeze to get the final disk position used for all calculations
	float2 SqueezedDistortedDisk = DistortedDisk;
	SqueezedDistortedDisk.x *= SqueezeFactor;

	// Compute the effective disk radius (after squeeze and Petzval distortion)
	// This is used for edge softness and boundary tests
	float EffectiveRadius = length(SqueezedDistortedDisk);

	// Compute blade mask factor (includes edge softness for blade shapes)
	float BladeMaskFactor = ComputeBladeMask(SqueezedDistortedDisk, BokehEdgeSoftness);

	// Compute bokeh weight and tint
	float3 SampleMultiplier = float3(1.0, 1.0, 1.0);

	if (BokehEnabled > 0)
	{
		// Map texture so that circular features (like a ring at radius 1) follow the distorted boundary
		// The boundary is computed as: NormDisk -> Petzval -> Squeeze
		// We want texture radius to equal the EffectiveRadius

		float DiskRadius = length(NormDisk);

		// Scale NormDisk so its length equals EffectiveRadius (the distance to distorted boundary)
		float2 BokehUVDisk = (DiskRadius > 0.0001)
			? NormDisk * (EffectiveRadius / DiskRadius)
			: float2(0.0, 0.0);

		// Convert to [0,1] UV coordinates
		float2 BokehUV = BokehUVDisk * 0.5 + 0.5;

		// Sample bokeh texture
		float4 BokehColor = BokehTexture.Sample(BokehSampler, BokehUV);

		// Extract weight from bokeh texture
		
		float Weight;
		
		if (BokehWeightChannel == 0) // Alpha determines weight
		{
			Weight = BokehColor.a;
		}
		else // Luminance determines weight
		{
			Weight = LuminanceSRGB(BokehColor.rgb);
		}
		
		Weight = saturate(Weight);

		// Apply Petzval cutoff: reject samples where distorted position exceeds unit disk
		if (Petzval != 0 && BokehShape == BOKEH_SHAPE_CIRCLE)
		{
			if (EffectiveRadius > 1.0)
			{
				Weight = 0.0;
			}
		}

		// Compute tint (preserving luminance to prevent darkening)
		
		float3 Tint = float3(1.0, 1.0, 1.0);
		
		if (BokehUseTint > 0)
		{
			// Blend between white and bokeh color
			float3 RawTint = lerp(float3(1.0, 1.0, 1.0), BokehColor.rgb, BokehTintStrength);

			// Calculate luminance of the tint
			float TintLuminance = LuminanceSRGB(RawTint);

			// Normalize tint to preserve brightness (only apply color shift, not brightness change)
			if (TintLuminance > 0.0001)
			{
				Tint = RawTint / TintLuminance;
			}
			else
			{
				Tint = RawTint;
			}
		}

		// Combine weight and tint
		SampleMultiplier = Tint * Weight;
	}
	else
	{
		// No bokeh texture
		SampleMultiplier = float3(1.0, 1.0, 1.0);
	}

	// Apply blade mask (includes edge softness)
	SampleMultiplier *= BladeMaskFactor;

	// Compute barrel occlusion (cat's eye bokeh)
	// Lerps color with black when occluded.
	float BarrelOcclusion = ComputeBarrelOcclusion(UV, ApertureOffsetCm, ApertureRadiusCm, BokehEdgeSoftness);

	// Apply spectral weight and AA weight for axial chromatic aberration and reconstruction
	float3 SpectralSampleMultiplier = SampleMultiplier * AntiAliasingWeight * SpectralWeight * BarrelOcclusion;

	// Weighted additive accumulation
	OutColor.rgb = PrevAccum.rgb + Sample.rgb * SpectralSampleMultiplier;

	// Alpha: accumulate weight scaled by AntiAliasingWeight and avg(SpectralWeight) for correct normalization.
	float AvgSpectralWeight = (SpectralWeight.r + SpectralWeight.g + SpectralWeight.b) / 3.0;
	float WeightToAccumulate = (SampleMultiplier.r + SampleMultiplier.g + SampleMultiplier.b) / 3.0 * AntiAliasingWeight * AvgSpectralWeight;

	OutColor.a = PrevAccum.a + WeightToAccumulate;
}
