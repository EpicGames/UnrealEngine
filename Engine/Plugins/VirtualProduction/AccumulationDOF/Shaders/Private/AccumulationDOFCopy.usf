// Copyright Epic Games, Inc. All Rights Reserved.

//
// Simple copy shader used when 32 <-> 16 bit pixel format conversion is needed (we accumulate in 32-bit)
//


#include "/Engine/Public/Platform.ush"


Texture2D SourceTexture;
SamplerState SourceSampler;

// Dithering parameters for f32->f16 format conversion
uint FrameCounter;
float DitherStrength;

// Source UV crop region (for overscan cropping)
float2 SourceUVMin;
float2 SourceUVMax;

// Interleaved gradient noise (IGN) for temporal dithering
float InterleavedGradientNoise(float2 PixelPos, uint FrameId)
{
	float FrameOffset = float(FrameId) * 1.61803398875; // Golden ratio for good temporal distribution
	return frac(52.9829189 * frac(0.06711056 * PixelPos.x + 0.00583715 * PixelPos.y + FrameOffset));
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2         UV : TEXCOORD0,
	out float4   OutColor : SV_Target0
)
{
	// Remap UV from [0,1] to [SourceUVMin, SourceUVMax] for cropping
	// When SourceUVMin = (0,0) and SourceUVMax = (1,1), this is a no-op
	float2 SourceUV = lerp(SourceUVMin, SourceUVMax, UV);
	float4 SourceColor = SourceTexture.Sample(SourceSampler, SourceUV);

	// Apply temporal dithering when converting to lower precision formats (f32 -> f16)
	if (DitherStrength > 0.0)
	{
		float Noise = InterleavedGradientNoise(SvPosition.xy, FrameCounter);
		float Dither = (Noise - 0.5) * DitherStrength;
		OutColor.rgb = SourceColor.rgb + Dither;
		OutColor.a = SourceColor.a;
	}
	else
	{
		// No dithering needed (format conversion is lossless or not required)
		OutColor = SourceColor;
	}
}
