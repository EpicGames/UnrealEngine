// Copyright Epic Games, Inc. All Rights Reserved.

//
// Injects accumulated AccumulationDOF result into scene color for final post-processing
//

// Copies the accumulated DOF texture to the scene color buffer, handling
// potential resolution/viewport mismatches (through bilinear sampling).
// Optionally draws a progress bar overlay for preview mode.


#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/MiniFontCommon.ush"


// Character definitions to complement those in MiniFontCommon

#define _LPAREN_ 40
#define _RPAREN_ 41

#define _a_ 97
#define _b_ 98
#define _c_ 99
#define _d_ 100
#define _e_ 101
#define _f_ 102
#define _g_ 103
#define _h_ 104
#define _i_ 105
#define _j_ 106
#define _k_ 107
#define _l_ 108
#define _m_ 109
#define _n_ 110
#define _o_ 111
#define _p_ 112
#define _q_ 113
#define _r_ 114
#define _s_ 115
#define _t_ 116
#define _u_ 117
#define _v_ 118
#define _w_ 119
#define _x_ 120
#define _y_ 121
#define _z_ 122

Texture2D AccumulatedTexture;
SamplerState AccumulatedSampler;

float2 AccumulatedTextureSize;  // Accumulated texture dimensions for aspect ratio letterboxing
float2 OutputTextureSize;       // Output texture dimensions

float ProgressBarFraction;  // 0.0 to 1.0
uint bDrawProgressBar;      // Non-zero to draw progress bar
float2 ViewportSize;        // Viewport dimensions in pixels
float OverscanFraction;     // Overscan fraction for progress bar positioning (0.0 = none, 0.1 = 10%)
uint bDrawPreviewLabel;     // Non-zero to draw "Accumulation DOF Preview" label
uint bIsFrozen;             // Non-zero to append "(FROZEN)" to preview label
uint bApplyAspectFit;       // Non-zero to letterbox/pillarbox when accumulator and viewport aspects differ

// Lens distortion (optional). When bApplyLensDistortion != 0, the source UV is offset by
// the undistorting displacement before sampling the accumulated buffer. UI overlays remain
// in screen space.

Texture2D UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;
uint bApplyLensDistortion;

// Dithering parameters to counter tone curve quantization, although the 64-bit one is working out well
uint FrameCounter;
float DitherStrength;

// Interleaved gradient noise for temporal dithering
float InterleavedGradientNoise(float2 PixelPos, uint FrameId)
{
	float FrameOffset = float(FrameId) * 1.61803398875;
	return frac(52.9829189 * frac(0.06711056 * PixelPos.x + 0.00583715 * PixelPos.y + FrameOffset));
}

void MainVS(
	in  uint   VertexId    : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV       : TEXCOORD0
)
{
	// Full-screen triangle
	OutUV = float2((VertexId << 1) & 2, VertexId & 2);
	OutPosition = float4(OutUV * float2(2, -2) + float2(-1, 1), 0, 1);
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2 UV         : TEXCOORD0,
	out float4 OutColor   : SV_Target0
)
{
	// Remap UV to maintain source aspect ratio
	float2 RemappedUV = UV;
	bool bOutsideLetterbox = false;

	// Letterbox/pillarbox the accumulator into the viewport when their aspects differ. Skipped
	// when the caller's ViewRect is already shrunk to the target aspect.
	if (bApplyAspectFit != 0)
	{
		// Compute aspect ratios to handle letterboxing when camera aspect differs from the viewport's
		float AccumAspect = AccumulatedTextureSize.x / AccumulatedTextureSize.y;
		float ViewportAspect = ViewportSize.x / ViewportSize.y;

		if (AccumAspect > ViewportAspect + 0.001)
		{
			// When source is wider, use top/bottom black bars

			float LetterboxHeight = ViewportAspect / AccumAspect;
			float LetterboxMargin = (1.0 - LetterboxHeight) * 0.5;

			RemappedUV.y = (UV.y - LetterboxMargin) / LetterboxHeight;
			bOutsideLetterbox = (UV.y < LetterboxMargin || UV.y > 1.0 - LetterboxMargin);
		}
		else if (AccumAspect < ViewportAspect - 0.001)
		{
			// When source is taller, use left/right black bars

			float PillarboxWidth = AccumAspect / ViewportAspect;
			float PillarboxMargin = (1.0 - PillarboxWidth) * 0.5;

			RemappedUV.x = (UV.x - PillarboxMargin) / PillarboxWidth;
			bOutsideLetterbox = (UV.x < PillarboxMargin || UV.x > 1.0 - PillarboxMargin);
		}
	}

	if (bOutsideLetterbox)
	{
		OutColor = float4(0, 0, 0, 1);
		return;
	}

	// Apply lens distortion warp. Sample the LUT at the source-space UV so it stays in
	// the accumulated buffer's coordinate frame when aspect-fit is letterboxing.
	if (bApplyLensDistortion != 0)
	{
		float2 Displacement = UndistortingDisplacementTexture.SampleLevel(UndistortingDisplacementSampler, RemappedUV, 0).rg;
		RemappedUV += Displacement;
	}

	OutColor = AccumulatedTexture.Sample(AccumulatedSampler, RemappedUV);

	// Apply conservative dithering to prevent banding from tone curve quantization
	if (DitherStrength > 0.0)
	{
		float Noise = InterleavedGradientNoise(SvPosition.xy, FrameCounter);
		float Dither = (Noise - 0.5) * DitherStrength;
		OutColor.rgb += Dither;
	}

	// Calculate visible region accounting for overscan crop
	float CropMargin = OverscanFraction / (2.0 * (1.0 + OverscanFraction));
	float VisibleBottom = 1.0 - CropMargin;

	// Size unit for consistent appearance across resolutions
	float Size = 50.0 / ViewportSize.y;

	// Draw progress bar overlay if enabled (similar to path tracer style)
	if (bDrawProgressBar && ProgressBarFraction < 1.0)
	{
		// Progress bar dimensions scaled to resolution
		float2 ProgressSize = float2(2.0 * Size * ViewportSize.y / ViewportSize.x, 0.15 * Size);

		// Center at bottom of VISIBLE region (accounting for overscan crop)
		float2 Center = float2(0.5, VisibleBottom - Size * 0.75);
		float2 P = UV - Center;

		// Check if inside progress bar rect
		if (all(abs(P) < ProgressSize))
		{
			// Normalize to 0-1 within bar
			float2 BarUV = (P + ProgressSize) / (ProgressSize * 2.0);

			// Anti-aliased fill edge using screen-space derivatives
			float Blend = smoothstep(ProgressBarFraction - fwidth(BarUV.x),
			                         ProgressBarFraction + fwidth(BarUV.x), BarUV.x);

			// Red (filled) to grey (empty) - matches path tracer style
			float3 BarColor = lerp(float3(1.0, 0.1, 0.1), float3(0.5, 0.5, 0.5), Blend);

			// 75% opacity blend
			OutColor.rgb = lerp(OutColor.rgb, BarColor, 0.75);
		}
	}

	// Draw "Accumulation DOF Preview" label centered below progress bar
	// Appends " (FROZEN)" when frozen to indicate scene changes are ignored
	if (bDrawPreviewLabel)
	{
		// Label: "Accumulation DOF Preview" (24 chars) or "Accumulation DOF Preview (FROZEN)" (33 chars)
		const int BaseLabelLength = 24;
		const int FrozenSuffixLength = 9; // " (FROZEN)"
		const int CharWidth = NATIVE_CHARACTER_RES;
		int LabelLength = bIsFrozen ? (BaseLabelLength + FrozenSuffixLength) : BaseLabelLength;
		int LabelWidthPx = LabelLength * CharWidth;

		int2 PixelPos = int2(SvPosition.xy);

		int LabelCenterX = (int)(ViewportSize.x * 0.5);
		int LabelTopY = (int)(VisibleBottom * ViewportSize.y - Size * ViewportSize.y * 0.5);
		int LabelLeft = LabelCenterX - LabelWidthPx / 2;

		int2 Cursor = int2(LabelLeft, LabelTopY);
		float3 FontColor = float3(1.0, 1.0, 1.0);

		// "Accumulation DOF Preview"
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _A_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _c_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _c_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _u_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _m_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _u_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _l_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _a_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _t_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _i_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _o_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _n_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _SPC_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _D_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _O_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _F_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _SPC_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _P_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _r_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _e_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _v_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _i_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _e_);
		PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _w_);

		// " (FROZEN)"
		if (bIsFrozen)
		{
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _SPC_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _LPAREN_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _F_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _R_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _O_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _Z_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _E_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _N_);
			PrintCharacter(PixelPos, OutColor.rgb, FontColor, Cursor, _RPAREN_);
		}
	}
}
