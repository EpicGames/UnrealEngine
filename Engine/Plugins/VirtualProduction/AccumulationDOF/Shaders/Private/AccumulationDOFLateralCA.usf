// Copyright Epic Games, Inc. All Rights Reserved.

//
// Spectral lateral chromatic aberration shader
//

// Simulates wavelength-dependent radial displacement caused by thin lens refraction.
// Blue light bends more than red, so blue shifts outward from center while red shifts inward.
//
// The shader samples the image NUM_SPECTRAL_BANDS times across the visible spectrum. Each sample
// is offset radially by a dispersion factor, blue outward, red inward, green stays put.
// A physically-based LUT (CIE 1931 color matching) weights each sample's contribution to R/G/B channels. 
//
// A rectangular dead zone near frame center (controlled by StartOffset) matching the engine's normal 
// chromatic aberration.


#include "/Engine/Public/Platform.ush"

Texture2D SampleTexture;
SamplerState SampleSampler;

// Intensity in UV space
float IntensityUV;

// Start offset threshold per axis (0-1). Creates a rectangular dead zone in the center where no dispersion occurs.
float StartOffset;

// Number of spectral bands. Could probably use less than 19...
#define NUM_SPECTRAL_BANDS 19

// Small value to avoid division by zero
static const float Epsilon = 1e-6;

// Center in UV space (typically 0.5, 0.5)
float2 Center;

// LUT: wavelength 380-700nm -> normalized linear sRGB (R+G+B = 1)
// Per "Simple Analytic Approximations to the CIE XYZ Color Matching Functions" by Wyman et al.
//
// To re-generate or change the size, use generate_spectral_lut.py
//
// @todo: Account for working color space
//

#define SPECTRAL_LUT_SIZE 32
#define SPECTRAL_LUT_MAX_INDEX (SPECTRAL_LUT_SIZE - 1)

static const float3 SpectralLUT[SPECTRAL_LUT_SIZE] = {
	float3(0.175237, 0.000000, 0.824763), // 380.0 nm
	float3(0.167134, 0.000000, 0.832866), // 390.3 nm
	float3(0.149273, 0.000000, 0.850727), // 400.6 nm
	float3(0.113597, 0.000000, 0.886403), // 411.0 nm
	float3(0.062576, 0.000000, 0.937424), // 421.3 nm
	float3(0.017058, 0.046903, 0.936039), // 431.6 nm
	float3(0.000000, 0.149028, 0.850972), // 441.9 nm
	float3(0.000000, 0.258459, 0.741541), // 452.3 nm
	float3(0.000000, 0.360560, 0.639440), // 462.6 nm
	float3(0.000000, 0.458963, 0.541037), // 472.9 nm
	float3(0.000000, 0.558498, 0.441502), // 483.2 nm
	float3(0.000000, 0.660484, 0.339516), // 493.5 nm
	float3(0.000000, 0.763419, 0.236581), // 503.9 nm
	float3(0.000000, 0.860713, 0.139287), // 514.2 nm
	float3(0.000000, 0.940980, 0.059020), // 524.5 nm
	float3(0.075869, 0.924131, 0.000000), // 534.8 nm
	float3(0.225498, 0.774502, 0.000000), // 545.2 nm
	float3(0.359627, 0.640373, 0.000000), // 555.5 nm
	float3(0.479421, 0.520579, 0.000000), // 565.8 nm
	float3(0.585598, 0.414402, 0.000000), // 576.1 nm
	float3(0.680139, 0.319861, 0.000000), // 586.5 nm
	float3(0.763553, 0.236447, 0.000000), // 596.8 nm
	float3(0.835816, 0.164184, 0.000000), // 607.1 nm
	float3(0.896155, 0.103845, 0.000000), // 617.4 nm
	float3(0.943269, 0.056731, 0.000000), // 627.7 nm
	float3(0.975068, 0.024932, 0.000000), // 638.1 nm
	float3(0.992392, 0.007608, 0.000000), // 648.4 nm
	float3(0.998694, 0.001306, 0.000000), // 658.7 nm
	float3(0.999918, 0.000082, 0.000000), // 669.0 nm
	float3(1.000000, 0.000000, 0.000000), // 679.4 nm
	float3(1.000000, 0.000000, 0.000000), // 689.7 nm
	float3(1.000000, 0.000000, 0.000000), // 700.0 nm
};

// Compute spectral RGB weight for normalized spectral position using our physically-based LUT.
// Maps [0,1] to wavelength 380-700nm with linear interpolation.
float3 SpectralWeight(float NormalizedSpectralPosition)
{
	float IndexFloat = saturate(NormalizedSpectralPosition) * SPECTRAL_LUT_MAX_INDEX;
	int Index0 = (int)IndexFloat;
	int Index1 = min(Index0 + 1, SPECTRAL_LUT_MAX_INDEX);
	float T = IndexFloat - (float)Index0;
	return lerp(SpectralLUT[Index0], SpectralLUT[Index1], T);
}

void MainVS(
	in  uint   VertexId    : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV       : TEXCOORD0
)
{
	// Full-screen triangle
	OutUV = float2((VertexId << 1) & 2, VertexId & 2);
	OutPosition = float4(OutUV * float2(2, -2) + float2(-1, 1), 0, 1);
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2 UV         : TEXCOORD0,
	out float4 OutColor   : SV_Target0
)
{
	// Convert UV (0-1) to lens-centered coordinates (-1 to 1)
	float2 LensUV = (UV - Center) * 2.0;

	// Per-axis offset from dead zone boundary to pixel.
	// Matches Engine's chromatic aberration: saturate(abs(LensUV) - StartOffset) per axis.
	// This creates a rectangular dead zone. See PostProcessTonemap.usf.
	float2 RadialOffsetFromDeadZone = sign(LensUV) * saturate(abs(LensUV) - StartOffset);

	// Magnitude of the offset from dead zone
	float OffsetMagnitude = length(RadialOffsetFromDeadZone);

	// Normalized direction scaled back to UV space.
	// The * 0.5 compensates for the * 2.0 used to create LensUV.
	float2 DispersionDirection = OffsetMagnitude > Epsilon ? (RadialOffsetFromDeadZone / OffsetMagnitude) * 0.5 : float2(0.0, 0.0);

	// Per-pixel dispersion magnitude in UV space
	float DispersionMagnitude = IntensityUV * OffsetMagnitude;

	// Accumulate spectral bands with weighted contributions
	float3 AccumulatedColor = float3(0.0, 0.0, 0.0);
	float3 AccumulatedWeight = float3(0.0, 0.0, 0.0);

	// Loop over spectral bands accumulating sampled colors per the band weight for each color component
	UNROLL
	for (int BandIndex = 0; BandIndex < NUM_SPECTRAL_BANDS; BandIndex++)
	{
		// Spectral position: 0:0 blue end, 1: red end
		float SpectralPosition = (float)BandIndex / (float)(NUM_SPECTRAL_BANDS - 1);
		float3 BandWeight = SpectralWeight(SpectralPosition);

		// Dispersion: blue shifts outward (+0.5), red shifts inward (-0.5)
		// Range is [-0.5, 0.5] to match the Engine's span which offsets R/G from B which remains at 0.
		float DispersionFactor = 0.5 - SpectralPosition;

		// Sample at dispersed UV position
		float2 DispersedUV = UV + DispersionDirection * (DispersionMagnitude * DispersionFactor);
		float3 SampledColor = SampleTexture.Sample(SampleSampler, DispersedUV).rgb;

		// Weighted accumulation per channel
		AccumulatedColor += SampledColor * BandWeight;
		AccumulatedWeight += BandWeight;
	}

	// Normalize by total weight per channel
	// This maintains neutrals as expected (i.e. lateral chromatic aberration of a gray frame remains gray)
	OutColor = float4(AccumulatedColor / max(AccumulatedWeight, Epsilon), 1.0);
}
