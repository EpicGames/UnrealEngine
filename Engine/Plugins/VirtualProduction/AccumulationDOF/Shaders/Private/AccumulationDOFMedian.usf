// Copyright Epic Games, Inc. All Rights Reserved.

//
// 3x3 Median filter for noise reduction
//

// See DOFPostfiltering.usf for referece

#include "/Engine/Private/Common.ush"

// Input texture
Texture2D InputTexture;
SamplerState InputSampler;

// Texture dimensions for offset calculation
float2 InputTexelSize; // 1.0 / TextureSize


void MainVS(
	in  uint   VertexId    : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV       : TEXCOORD0)
{
	// Full-screen triangle
	OutUV = float2((VertexId << 1) & 2, VertexId & 2);
	OutPosition = float4(OutUV * float2(2.0, -2.0) + float2(-1.0, 1.0), 0.0, 1.0);
}

static const int2 kOffsets3x3[9] = {
	int2(-1, -1),
	int2( 0, -1),
	int2( 1, -1),
	int2(-1,  0),
	int2( 0,  0),
	int2( 1,  0),
	int2(-1,  1),
	int2( 0,  1),
	int2( 1,  1),
};

// Operator that outputs lowest, median and highest values from 3 input values
void LMHOperator(float4 A, float4 B, float4 C, out float4 L, out float4 M, out float4 H)
{
	float4 X = min(B, C);
	float4 Y = max(B, C);

	L = min(A, X);
	float4 Z = max(A, X);

	M = min(Z, Y);
	H = max(Z, Y);
}

// 3 samples median
float4 Median3(float4 A, float4 B, float4 C)
{
	float4 L, M, H;
	LMHOperator(A, B, C, L, M, H);
	return M;
}

// Finds the median in O(n) comparisons (uses Smith1996 algo)
float4 Median9(float4 In[9])
{
	// First layer: sort each row of 3
	float4 L0[3];
	float4 M0[3];
	float4 H0[3];
	for (uint j = 0; j < 3; j++)
	{
		LMHOperator(In[j * 3 + 0], In[j * 3 + 1], In[j * 3 + 2], L0[j], M0[j], H0[j]);
	}

	// Second layer: find candidates for median
	float4 M1[3];
	M1[0] = max(max(L0[0], L0[1]), L0[2]);  // Max of minimums
	M1[1] = Median3(M0[0], M0[1], M0[2]);   // Median of medians
	M1[2] = min(min(H0[0], H0[1]), H0[2]);  // Min of maximums

	// Third layer: median of the three candidates
	return Median3(M1[0], M1[1], M1[2]);
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2 UV         : TEXCOORD0,
	out float4 OutColor   : SV_Target0)
{
	// Sample 3x3 neighborhood
	float4 Samples[9];
	for (uint i = 0; i < 9; i++)
	{
		float2 SampleUV = UV + float2(kOffsets3x3[i]) * InputTexelSize;
		// Clamp to valid UV range
		SampleUV = clamp(SampleUV, InputTexelSize * 0.5, 1.0 - InputTexelSize * 0.5);
		Samples[i] = InputTexture.SampleLevel(InputSampler, SampleUV, 0);
	}

	// Compute median
	OutColor = Median9(Samples);
}
