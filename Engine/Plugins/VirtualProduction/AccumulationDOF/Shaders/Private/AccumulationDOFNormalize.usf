// Copyright Epic Games, Inc. All Rights Reserved.

//
// Normalization of accumulated weighted samples
//

#include "/Engine/Public/Platform.ush"

Texture2D AccumTexture;
SamplerState AccumSampler;

float InvWeightSum;

void MainVS(
	in  uint   VertexId    : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV       : TEXCOORD0
)
{
	// Full-screen triangle
	OutUV = float2((VertexId << 1) & 2, VertexId & 2);
	OutPosition = float4(OutUV * float2(2, -2) + float2(-1, 1), 0, 1);
}

void MainPS(
	in  float4 SvPosition : SV_POSITION,
	in  float2 UV         : TEXCOORD0,
	out float4 OutColor   : SV_Target0
)
{
	// Sample accumulated buffer
	float4 Accum = AccumTexture.Sample(AccumSampler, UV);

	// Alpha channel contains the sum of weights for this pixel
	float PixelWeightSum = Accum.a;

	if (PixelWeightSum > 0.0001)
	{
		OutColor.rgb = Accum.rgb / PixelWeightSum;
		OutColor.a = 0.0; // opaque
	}
	else
	{
		// No samples contributed to this pixel
		OutColor = float4(0.0, 0.0, 0.0, 0.0);
	}
}
