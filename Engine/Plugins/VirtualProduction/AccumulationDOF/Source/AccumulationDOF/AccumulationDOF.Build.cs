// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AccumulationDOF : ModuleRules
{
	public AccumulationDOF(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateIncludePaths.AddRange(new string[] {
			Path.Combine(ModuleDirectory, "Private/Rendering"),
			Path.Combine(ModuleDirectory, "Private/Shaders"),
			Path.Combine(ModuleDirectory, "Private/Utils"),
		});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CinematicCamera",
				"Core",
				"Engine",
				"MovieRenderPipelineCore",
				"Slate",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"ImageWriteQueue",
				"MovieRenderPipelineRenderPasses",
				"Projects",
				"RenderCore",
				"Renderer",
				"RHI",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.Add("PropertyEditor");
		}
	}
}
