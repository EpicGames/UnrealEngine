// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOF.h"
#include "AccumulationDOFComponent.h"
#include "AccumulationDOFLog.h"
#include "CineCameraActor.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/Nodes/MovieGraphDeferredPassNode.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "MovieGraphAccumulationDOFModifierNode.h"
#include "MovieGraphAccumulationDOFPass.h"
#include "ShaderCore.h"

#if WITH_EDITOR
#include "MovieGraphAccumulationDOFModifierCustomization.h"
#include "PropertyEditorModule.h"
#endif // WITH_EDITOR

DEFINE_LOG_CATEGORY(LogAccumulationDOF);

#define LOCTEXT_NAMESPACE "FAccumulationDOFModule"

const FName FAccumulationDOFModule::MRGPassFactoryPassName(TEXT("AccumulationDOF"));

void FAccumulationDOFModule::StartupModule()
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("AccumulationDOF"));

	if (Plugin.IsValid())
	{
		FString PluginShaderDirectory = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/AccumulationDOF"), PluginShaderDirectory);
	}

#if WITH_EDITOR
	FPropertyEditorModule& PropertyModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("MovieGraphAccumulationDOFModifierNode",
		FOnGetDetailCustomizationInstance::CreateStatic(&FMovieGraphAccumulationDOFModifierCustomization::MakeInstance));
#endif	// WITH_EDITOR

	// Register AccumulationDOF pass factory with the deferred renderer node.
	// When a camera has an active UAccumulationDOFComponent, the deferred renderer will
	// use FMovieGraphAccumulationDOFPass instead of FMovieGraphDeferredPass.
	UMovieGraphDeferredRenderPassNode::RegisterPassInstanceFactory(
		MRGPassFactoryPassName,
		[](const UMovieGraphDeferredRenderPassNode::FPassInstanceFactoryContext& InContext)
		-> TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase>
	{
		if (!InContext.Renderer)
		{
			return nullptr;
		}

		const TSharedPtr<const FMovieGraphRenderCameraSource> CameraSource = InContext.LayerData.RenderPassNode.IsValid()
			? InContext.LayerData.RenderPassNode->GetRenderCameraSource()
			: nullptr;

		const UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = InContext.Renderer->GetCameraInfo(InContext.LayerData.CameraIndex, CameraSource.Get());
		const ACineCameraActor* CineCamera = Cast<ACineCameraActor>(CameraInfo.ViewActor);
		if (!CineCamera)
		{
			return nullptr;
		}

		// Use the component's current active state if available, otherwise default to disabled
		const UAccumulationDOFComponent* DOFComponent = CineCamera->FindComponentByClass<UAccumulationDOFComponent>();
		bool bDOFEnabled = DOFComponent && DOFComponent->IsActive();

		// Check if an Accumulation DOF modifier on this layer is overriding the enable state.
		// This allows enabling Accumulation DOF even if no component exists on the camera,
		// or disabling it even if the component is active. Checking the modifier is not ideal.
		// However, the modifier is applied *after* pass instantiation, so we cannot solely
		// rely on the component active state.
		if (InContext.EvaluatedConfig)
		{
			constexpr bool bIncludeCDOs = false;
			constexpr bool bExactMatch = true;
			const UMovieGraphAccumulationDOFModifierNode* ModNode =
				InContext.EvaluatedConfig->GetSettingForBranch<UMovieGraphAccumulationDOFModifierNode>(
					InContext.LayerData.BranchName, bIncludeCDOs, bExactMatch);

			if (ModNode
				&& ModNode->bOverride_EnableAccumulationDepthOfField
				&& ModNode->EnableAccumulationDepthOfField.Type == EMovieGraphAccumulationDOFEnableType::CustomValue)
			{
				bDOFEnabled = ModNode->EnableAccumulationDepthOfField.Value;
			}
		}

		if (bDOFEnabled)
		{
			return MakeUnique<FMovieGraphAccumulationDOFPass>();
		}

		return nullptr;
	});
}

void FAccumulationDOFModule::ShutdownModule()
{
#if WITH_EDITOR
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomClassLayout("MovieGraphAccumulationDOFModifierNode");
	}
#endif	// WITH_EDITOR

	UMovieGraphDeferredRenderPassNode::UnregisterPassInstanceFactory(MRGPassFactoryPassName);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAccumulationDOFModule, AccumulationDOF)
