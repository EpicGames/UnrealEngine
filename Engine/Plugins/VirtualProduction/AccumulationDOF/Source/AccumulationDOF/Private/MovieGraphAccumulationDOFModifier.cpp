// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAccumulationDOFModifier.h"

#include "AccumulationDOFComponent.h"
#include "Camera/CameraActor.h"
#include "EngineUtils.h"

FMovieGraphAccumulationDOFModifier_ComponentState::FMovieGraphAccumulationDOFModifier_ComponentState(UAccumulationDOFComponent* InComponent)
	: DOFComponent(InComponent)
{
	if (DOFComponent.IsValid())
	{
		bEnableAccumulationDepthOfField = DOFComponent->IsActive();
		NumSamples = DOFComponent->NumSamples;
		DOFSplatSize = DOFComponent->DOFSplatSize;
	}
}

void UMovieGraphAccumulationDOFModifier::ApplyModifier(const UWorld* World)
{
	OriginalComponentState.Reset();

	for (TActorIterator<ACameraActor> It(World); It; ++It)
	{
		ACameraActor* CameraActor = *It;
		if (!CameraActor)
		{
			continue;
		}

		UAccumulationDOFComponent* AccumulationDOFComponent = CameraActor->FindComponentByClass<UAccumulationDOFComponent>();
		bool bComponentWasAdded = false;

		if (!AccumulationDOFComponent)
		{
			// Only spawn a component when the modifier is explicitly enabling DOF. Any other case
			// (no enable override, CameraActorDefault, or CustomValue=false) has nothing to apply.
			const bool bWantsEnable =
				bOverride_EnableDOFComponents
				&& EnableDOFComponents.Type == EMovieGraphAccumulationDOFEnableType::CustomValue
				&& EnableDOFComponents.Value;

			if (!bWantsEnable)
			{
				continue;
			}

			AccumulationDOFComponent = NewObject<UAccumulationDOFComponent>(CameraActor, NAME_None, RF_Transient);
			AccumulationDOFComponent->RegisterComponent();
			bComponentWasAdded = true;
		}

		// First, cache its current state so UndoModifier can revert.
		FMovieGraphAccumulationDOFModifier_ComponentState CachedState(AccumulationDOFComponent);
		CachedState.bComponentWasAdded = bComponentWasAdded;
		OriginalComponentState.Add(CachedState);

		// Second, apply new state.
		{
			if (bOverride_EnableDOFComponents)
			{
				if (EnableDOFComponents.Type == EMovieGraphAccumulationDOFEnableType::CustomValue)
				{
					AccumulationDOFComponent->SetActive(EnableDOFComponents.Value);
				}
			}

			if (bOverride_NumSamples)
			{
				AccumulationDOFComponent->NumSamples = NumSamples;
			}

			if (bOverride_DOFSplatSize)
			{
				AccumulationDOFComponent->DOFSplatSize = DOFSplatSize;
			}
		}

		AccumulationDOFComponent->MarkRenderStateDirty();
	}
}

void UMovieGraphAccumulationDOFModifier::UndoModifier()
{
	for (const FMovieGraphAccumulationDOFModifier_ComponentState& ComponentState : OriginalComponentState)
	{
		if (UAccumulationDOFComponent* CachedDOFComponent = ComponentState.DOFComponent.Get())
		{
			if (ComponentState.bComponentWasAdded)
			{
				CachedDOFComponent->DestroyComponent();
				continue;
			}

			if (bOverride_EnableDOFComponents)
			{
				CachedDOFComponent->SetActive(ComponentState.bEnableAccumulationDepthOfField);
			}

			if (bOverride_NumSamples)
			{
				CachedDOFComponent->NumSamples = ComponentState.NumSamples;
			}

			if (bOverride_DOFSplatSize)
			{
				CachedDOFComponent->DOFSplatSize = ComponentState.DOFSplatSize;
			}

			CachedDOFComponent->MarkRenderStateDirty();
		}
	}
	
	OriginalComponentState.Reset();
}

FText UMovieGraphAccumulationDOFModifier::GetModifierName()
{
	static const FText ModifierName = NSLOCTEXT("MovieGraph", "AccumulationDOFModifierName", "Accumulation DOF");
	return ModifierName;
}
