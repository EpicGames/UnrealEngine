// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "MovieGraphAccumulationDOFModifierNode.h"

#include "MovieGraphAccumulationDOFModifier.generated.h"

class UAccumulationDOFComponent;

#define UE_API ACCUMULATIONDOF_API

/** The state of an Accumulation DOF component before modification. */
USTRUCT()
struct FMovieGraphAccumulationDOFModifier_ComponentState
{
	GENERATED_BODY()

	FMovieGraphAccumulationDOFModifier_ComponentState() = default;
	explicit FMovieGraphAccumulationDOFModifier_ComponentState(UAccumulationDOFComponent* InComponent);

	/** The Accumulation DOF component that the below cached state is for. */
	TWeakObjectPtr<UAccumulationDOFComponent> DOFComponent;

	/** The original enable/disable state of the Accumulation DOF component. */
	bool bEnableAccumulationDepthOfField = false;

	/** The original number of samples set on the Accumulation DOF component. */
	int32 NumSamples = 0;

	/** The original splat size set on the Accumulation DOF component. */
	float DOFSplatSize = 0.f;

	/** True if ApplyModifier created this component; UndoModifier will destroy it. */
	bool bComponentWasAdded = false;
};

/**
 * Updates all Accumulation DOF components within the world to the settings specified on this modifier.
 */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphAccumulationDOFModifier final : public UMovieGraphModifierBase
{
	GENERATED_BODY()

public:
	UMovieGraphAccumulationDOFModifier() = default;

	//~ Begin UMovieGraphModifierBase Interface
	UE_API virtual void ApplyModifier(const UWorld* World) override;
	UE_API virtual void UndoModifier() override;
	UE_API virtual FText GetModifierName() override;
	//~ End UMovieGraphModifierBase Interface

public:
	/** Whether "EnableDOFComponents" state should be applied. */
	UPROPERTY()
	bool bOverride_EnableDOFComponents = false;

	/** Whether "NumSamples" state should be applied. */
	UPROPERTY()
	bool bOverride_NumSamples = false;

	/** Whether "DOFSplatSize" state should be applied. */
	UPROPERTY()
	bool bOverride_DOFSplatSize = false;

	/** The enable/disable state of the Accumulation DOF component. */
	UPROPERTY()
	FMovieGraphAccumulationDOFEnableState EnableDOFComponents;

	/** The number of samples that the Accumulation DOF component should use. */
	UPROPERTY()
	int32 NumSamples = 0;

	/** The splat size that the Accumulation DOF component should use. */
	UPROPERTY()
	float DOFSplatSize = 0.f;

private:
	/** Tracks the original component state before the modifier was applied. */
	UPROPERTY(Transient)
	TArray<FMovieGraphAccumulationDOFModifier_ComponentState> OriginalComponentState;
};

#undef UE_API