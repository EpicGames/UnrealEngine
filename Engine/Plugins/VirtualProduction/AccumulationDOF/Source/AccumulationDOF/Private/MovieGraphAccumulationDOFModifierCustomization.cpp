// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAccumulationDOFModifierCustomization.h"

#if WITH_EDITOR

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "MovieGraphAccumulationDOFModifierNode.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MovieGraphAccumulationDOFModifierCustomization"

TSharedRef<IDetailCustomization> FMovieGraphAccumulationDOFModifierCustomization::MakeInstance()
{
	return MakeShared<FMovieGraphAccumulationDOFModifierCustomization>();
}

void FMovieGraphAccumulationDOFModifierCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	InDetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// The customization only supports editing a single Modifier node
	bool bFoundModifierNode = false;
	for (const TWeakObjectPtr<UObject>& CustomizedObject : CustomizedObjects)
	{
		if (CustomizedObject.IsValid() && CustomizedObject->IsA<UMovieGraphAccumulationDOFModifierNode>())
		{
			bFoundModifierNode = true;
			break;
		}
	}

	if (!bFoundModifierNode)
	{
		return;
	}

	// Add a "Collections" category with a custom whole-row widget which indicates that all cameras will be matched
	IDetailCategoryBuilder& CollectionsCategory =
		InDetailBuilder.EditCategory(FName("Collections"), FText::GetEmpty(), ECategoryPriority::Important);

	CollectionsCategory.AddCustomRow(FText::GetEmpty())
	.WholeRowWidget
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Margin(FMargin(0, 5, 0, 5))
		.Text(LOCTEXT("AllCameras", "Modifies the Accumulation DOF component on all active cameras"))
		.Font(IDetailLayoutBuilder::GetDetailFontItalic())
	];
}

void FMovieGraphAccumulationDOFModifierCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailBuilder = InDetailBuilder;
	CustomizeDetails(*InDetailBuilder);
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
