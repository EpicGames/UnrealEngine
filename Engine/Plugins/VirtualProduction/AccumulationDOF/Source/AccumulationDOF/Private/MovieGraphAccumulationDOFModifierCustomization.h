// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "IDetailCustomization.h"

class IMovieGraphModifierNodeInterface;

/** Customize how the Accumulation DOF modifier node appears in the details panel. */
class FMovieGraphAccumulationDOFModifierCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** The details builder associated with the customization. */
	TWeakPtr<IDetailLayoutBuilder> DetailBuilder;
};

#endif // WITH_EDITOR