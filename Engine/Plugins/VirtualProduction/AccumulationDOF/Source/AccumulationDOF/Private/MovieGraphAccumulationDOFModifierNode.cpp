// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAccumulationDOFModifierNode.h"

#include "MovieGraphAccumulationDOFModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphAccumulationDOFModifierNode)

#define LOCTEXT_NAMESPACE "MovieGraph"

UMovieGraphAccumulationDOFModifierNode::UMovieGraphAccumulationDOFModifierNode()
{
	Modifier = CreateDefaultSubobject<UMovieGraphAccumulationDOFModifier>(TEXT("AccumulationDOFModifier"));
}

#if WITH_EDITOR
FText UMovieGraphAccumulationDOFModifierNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = LOCTEXT("NodeName_AccumulationDOFModifier", "Accumulation DOF Modifier");

	return NodeName;
}

FText UMovieGraphAccumulationDOFModifierNode::GetMenuCategory() const
{
	return LOCTEXT("AccumulationDOFModifierNode_Category", "Utility");
}

FLinearColor UMovieGraphAccumulationDOFModifierNode::GetNodeTitleColor() const
{
	static const FLinearColor ModifierNodeColor = FLinearColor(0.6f, 0.113f, 0.113f);
	return ModifierNodeColor;
}

FSlateIcon UMovieGraphAccumulationDOFModifierNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon ModifierIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.ToggleActorPilotCameraView");

	OutColor = FLinearColor::White;
	return ModifierIcon;
}
#endif // WITH_EDITOR

TArray<UMovieGraphModifierBase*> UMovieGraphAccumulationDOFModifierNode::GetAllModifiers() const
{
	Modifier->bOverride_EnableDOFComponents = bOverride_EnableAccumulationDepthOfField;
	Modifier->bOverride_NumSamples = bOverride_NumSamples;
	Modifier->bOverride_DOFSplatSize = bOverride_DOFSplatSize;

	Modifier->EnableDOFComponents = EnableAccumulationDepthOfField;
	Modifier->NumSamples = NumSamples;
	Modifier->DOFSplatSize = DOFSplatSize;

	return { Modifier };
}

bool UMovieGraphAccumulationDOFModifierNode::SupportsCollections() const
{
	return false;
}

FString UMovieGraphAccumulationDOFModifierNode::GetNodeInstanceName() const
{
	return TEXT("AccumulationDOF");
}

#undef LOCTEXT_NAMESPACE // "MovieGraph"