// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAccumulationDOFPass.h"

#include "AccumulationDOFComponent.h"
#include "AccumulationDOFMath.h"
#include "AccumulationDOFSceneViewExtension.h"
#include "AccumulationDOFShaders.h"
#include "AccumulationDOFUtils.h"
#include "ApertureSampler.h"

#include "CanvasTypes.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineDataTypes.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "Styling/StyleColors.h"
#include "TextureResource.h"

DEFINE_LOG_CATEGORY(LogAccumulationDOFMRG);

FMovieGraphAccumulationDOFPass::FMovieGraphAccumulationDOFPass()
	: FMovieGraphDeferredPass()
{
}

FMovieGraphAccumulationDOFPass::~FMovieGraphAccumulationDOFPass()
{
	if (ApertureSampler)
	{
		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}
}

void FMovieGraphAccumulationDOFPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphDeferredPass::Setup(InRenderer, InRenderPassNode, InLayer);

	// Opt out of the base's per-CreateConfiguredView auto-restore. ADOF reuses each tile's capture view
	// state across many aperture-sample renders; per-call restore would clobber accumulating history.
	// Render() drives restore/backup explicitly once per tile.
	bAutoRestoreViewStateMirror = false;

	// For telemetry
	CachedActualNumSamples = 0;
	CachedEffectiveDOFSplatSize = 0.f;

	const UWorld* World = InRenderer->GetWorld();
	if (!World)
	{
		UE_LOGF(LogAccumulationDOFMRG, Warning, "Invalid world found. Accumulation DOF pass will not be set up properly.");
		return;
	}

	// Allocate dedicated injection view state per tile. Unlike the base class's SceneViewStates map (which
	// honors GetEnableHistoryPerTile()), the injection pass *always* needs per-tile history: it re-runs TAA,
	// Lumen and motion blur on the accumulated DOF result, and sharing one history across tiles produces
	// visible seams at tile boundaries.
	//
	// TileCount is captured here once and held in CurrentTileCount for the lifetime of the pass. ADOF does
	// not support mid-shot TileCount changes; Render() asserts the live value still matches.
	const UMovieGraphImagePassBaseNode* ParentNode = Cast<UMovieGraphImagePassBaseNode>(InLayer.RenderPassNode);
	check(ParentNode);
	CurrentTileCount = ParentNode->GetTileCount();
	for (int32 TileY = 0; TileY < CurrentTileCount.Y; ++TileY)
	{
		for (int32 TileX = 0; TileX < CurrentTileCount.X; ++TileX)
		{
			InjectionViewStates.Add(FIntPoint(TileX, TileY));
		}
	}
	for (TPair<FIntPoint, FSceneViewStateReference>& Pair : InjectionViewStates)
	{
		// Allocate after the map is populated
		Pair.Value.Allocate(World->GetFeatureLevel());
		if (FSceneViewStateInterface* ViewStateInterface = Pair.Value.GetReference())
		{
			if (World->Scene)
			{
				ViewStateInterface->AddLumenSceneData(World->Scene, 1.0f);
			}
		}
	}

	// Override LUT size to minimize banding
	constexpr bool bForceColorGradingLUTSize64 = true;
	SavedLUTSize = AccumulationDOFUtils::OverrideLUTSizeIfNeeded(bForceColorGradingLUTSize64);
}

void FMovieGraphAccumulationDOFPass::Teardown()
{
	if (ApertureSampler)
	{
		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}

	// Restore LUT size if it was overridden
	AccumulationDOFUtils::RestoreLUTSize(SavedLUTSize);
	SavedLUTSize = 0;

	CachedCineCameraComponent.Reset();
	CachedDOFComponent.Reset();
	CaptureExtension.Reset();
	InjectionExtension.Reset();

	for (TPair<FIntPoint, FSceneViewStateReference>& Pair : InjectionViewStates)
	{
		if (FSceneViewStateInterface* ViewStateInterface = Pair.Value.GetReference())
		{
			ViewStateInterface->ClearMIDPool();
		}
		Pair.Value.Destroy();
	}

	InjectionViewStates.Empty();
	InjectionStatesInitialized.Empty();

	FMovieGraphDeferredPass::Teardown();
}

void FMovieGraphAccumulationDOFPass::AddReferencedObjects(FReferenceCollector& Collector)
{
	FMovieGraphDeferredPass::AddReferencedObjects(Collector);

	// Protect ApertureSampler (and its UPROPERTY render targets) from GC.
	// This is critical because ForceGarbageCollection() is called during the
	// aperture sampling loop via ReleaseRenderingMemory().
	Collector.AddReferencedObject(ApertureSampler);

	// Protect the per-tile injection view states from garbage collection
	for (TPair<FIntPoint, FSceneViewStateReference>& Pair : InjectionViewStates)
	{
		if (FSceneViewStateInterface* InjectionViewRef = Pair.Value.GetReference())
		{
			InjectionViewRef->AddReferencedObjects(Collector);
		}
	}
}

void FMovieGraphAccumulationDOFPass::UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const
{
	if (CachedActualNumSamples <= 0)
	{
		return;
	}

	InOutTelemetry.bUsesAccumulationDepthOfField = true;
	InOutTelemetry.AccumulationDepthOfFieldSampleCount = FMath::Max(InOutTelemetry.AccumulationDepthOfFieldSampleCount, CachedActualNumSamples);
	InOutTelemetry.AccumulationDepthOfFieldSplatSize = FMath::Max(InOutTelemetry.AccumulationDepthOfFieldSplatSize, CachedEffectiveDOFSplatSize);
}

void FMovieGraphAccumulationDOFPass::UpdateCameraDependentState()
{
	// Update cached camera component
	if (!CachedCineCameraComponent.IsValid())
	{
		CachedCineCameraComponent = FindCineCameraComponent();
		FindAccumulationDOFComponent();
	}

	// Update sampler's camera state
	if (ApertureSampler && CachedCineCameraComponent.IsValid())
	{
		AccumulationDOF::FApertureSamplerCameraState CameraState = BuildCameraState();
		ApertureSampler->UpdateCameraState(CameraState);
		bCameraParamsInitialized = true;
	}
	else if (!bCameraParamsInitialized)
	{
		UE_LOGF(LogAccumulationDOFMRG, Warning, "No CineCameraActor found. Using default values.");
		bCameraParamsInitialized = true;
	}
}

void FMovieGraphAccumulationDOFPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
	CachedMotionBlurFraction = InTimeData.MotionBlurFraction;

	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	const FResolutionAndCameraInfo ResolutionAndCameraInfo = GetResolutionAndCameraInfo(InTimeData.EvaluatedConfig.Get());
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams RenderTargetInitParams = GetRenderTargetInitParams(InTimeData, ResolutionAndCameraInfo.BackbufferResolution);

	// Resolution invariance guard: if the per-tile BackbufferResolution has changed since the sampler was
	// initialized (e.g. graph variable changed TileCount mid-shot), tear it down so it re-initializes below.
	if (ApertureSampler && ApertureSampler->GetConfig().Resolution != ResolutionAndCameraInfo.BackbufferResolution)
	{
		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}

	// Initialize the sampler. This would ideally be done in Setup(), but we need access to the evaluated graph for this.
	if (!ApertureSampler)
	{
		InitializeSampler(InTimeData.EvaluatedConfig.Get());
	}

	if (!ApertureSampler || !ApertureSampler->IsInitialized())
	{
		UE_LOGF(LogAccumulationDOFMRG, Error, "ApertureSampler not initialized");
		return;
	}

	UpdateCameraDependentState();

	UMovieGraphImagePassBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);
	if (!ParentNodeThisFrame)
	{
		return;
	}

	// CurrentTileCount is locked at Setup time. A mid-shot change would desync InjectionViewStates from the
	// loop bounds AND poison the already-computed ResolutionAndCameraInfo (whose TileSize/BackbufferResolution
	// were derived from the live count via GetResolutionAndCameraInfo above). Bail.
	const FIntPoint LiveTileCount = ParentNodeThisFrame->GetTileCount();
	if (!ensureMsgf(LiveTileCount == CurrentTileCount,
		TEXT("AccumulationDOF: TileCount changed mid-shot from [%d,%d] to [%d,%d]. Frame skipped - mid-shot tile-count changes are unsupported."),
		CurrentTileCount.X, CurrentTileCount.Y, LiveTileCount.X, LiveTileCount.Y))
	{
		return;
	}

	const float TileOverlapFraction = ParentNodeThisFrame->GetTileOverlapPercentage() / 100.f;
	CurrentTileOverlappedPad = FIntPoint(
		FMath::CeilToInt(ResolutionAndCameraInfo.TileSize.X * TileOverlapFraction),
		FMath::CeilToInt(ResolutionAndCameraInfo.TileSize.Y * TileOverlapFraction));

	for (int32 TileY = 0; TileY < CurrentTileCount.Y; ++TileY)
	{
		for (int32 TileX = 0; TileX < CurrentTileCount.X; ++TileX)
		{
			CurrentTileIndex = FIntPoint(TileX, TileY);

			// Page-to-system-memory: explicitly restore this tile's capture view state before any rendering
			// (warmup or full). The base's auto-restore is suppressed via bAutoRestoreViewStateMirror=false;
			// doing it once per tile (instead of once per aperture sample) is why we opt out.
			const bool bMirrorViewStates = ParentNodeThisFrame->GetEnableHistoryPerTile()
				&& ParentNodeThisFrame->GetEnablePageToSystemMemory();
			if (bMirrorViewStates)
			{
				if (FSceneViewStateInterface* CaptureViewState = FMovieGraphDeferredPass::GetSceneViewState(ParentNodeThisFrame, CurrentTileIndex.X, CurrentTileIndex.Y))
				{
					CaptureViewState->SystemMemoryMirrorRestore(SystemMemoryMirror.Get());
				}
			}

			if (InTimeData.bDiscardOutput)
			{
				// During warmup frames, render a single aperture sample per tile to prime temporal history for each tile's view state.
				CurrentApertureSampleIndex = 0;

				UTextureRenderTarget2D* RenderTarget = GraphRenderer->GetOrCreateViewRenderTarget(RenderTargetInitParams, RenderDataIdentifier);
				FRenderTarget* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
				check(RenderTargetResource);

				UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();
				CameraInfo.bAllowCameraAspectRatio = true;
				CameraInfo.TilingParams.TileCount = CurrentTileCount;
				CameraInfo.TilingParams.TileIndexes = CurrentTileIndex;
				CameraInfo.TilingParams.TileSize = ResolutionAndCameraInfo.TileSize;
				CameraInfo.TilingParams.OverlapPad = CurrentTileOverlappedPad;
				CameraInfo.ProjectionMatrixJitterAmount = ComputeProjectionMatrixJitter(FVector2D::ZeroVector, ResolutionAndCameraInfo.BackbufferResolution);
				CameraInfo.bUseCameraManagerPostProcess = !ResolutionAndCameraInfo.bRenderAllCameras;
				{
					const FIntPoint OutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(InTimeData.EvaluatedConfig.Get(), ResolutionAndCameraInfo.CameraAspectRatio);
					ReapplyOverscanPreservingEngineScaling(CameraInfo, ResolutionAndCameraInfo.OverscanFraction, ResolutionAndCameraInfo.AccumulatorResolution, OutputResolution);
				}

				UE::MovieGraph::Rendering::FViewFamilyInitData ViewFamilyInitData;
				ViewFamilyInitData.RenderTarget = RenderTargetResource;
				ViewFamilyInitData.World = GraphRenderer->GetWorld();
				ViewFamilyInitData.TimeData = InTimeData;
				ViewFamilyInitData.SceneCaptureSource = ESceneCaptureSource::SCS_FinalToneCurveHDR;
				ViewFamilyInitData.bWorldIsPaused = false;
				ViewFamilyInitData.FrameIndex = InTimeData.RenderedFrameNumber;
				ViewFamilyInitData.AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
				ViewFamilyInitData.ShowFlags = ParentNodeThisFrame->GetShowFlags();
				ViewFamilyInitData.ViewModeIndex = ParentNodeThisFrame->GetViewModeIndex();
				ViewFamilyInitData.ProjectionMode = CameraInfo.ViewInfo.ProjectionMode;

				FCreateViewResult ViewResult = CreateConfiguredView(CameraInfo, ViewFamilyInitData, ResolutionAndCameraInfo.BackbufferResolution, ResolutionAndCameraInfo.AccumulatorResolution, ParentNodeThisFrame, CurrentTileIndex.X, CurrentTileIndex.Y);

				FCanvas Canvas = FCanvas(RenderTargetResource, nullptr, GraphRenderer->GetWorld(), GraphRenderer->GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
				GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewResult.ViewFamily.ToSharedPtr().Get());

				CurrentApertureSampleIndex = -1;
			}
			else
			{
				// Full render for this tile: reset accumulator, run all aperture samples + injection, submit per tile.
				ApertureSampler->ResetAccumulation();
				RenderSamples(InFrameTraversalContext, InTimeData, ResolutionAndCameraInfo, RenderTargetInitParams);
			}

			// Page-to-system-memory: backup this tile's capture view state after all renders for this tile
			// have submitted. Symmetric with the restore above. The injection view state lives in a separate
			// per-tile map and has its own backup inside RenderInjectionAndSubmit.
			if (bMirrorViewStates)
			{
				if (FSceneViewStateInterface* CaptureViewState = FMovieGraphDeferredPass::GetSceneViewState(ParentNodeThisFrame, CurrentTileIndex.X, CurrentTileIndex.Y))
				{
					CaptureViewState->SystemMemoryMirrorBackup(SystemMemoryMirror.Get());
				}
			}
		}
	}
}

TSharedRef<FSceneViewFamilyContext> FMovieGraphAccumulationDOFPass::CreateSceneViewFamily(const UE::MovieGraph::Rendering::FViewFamilyInitData& InInitData) const
{
	TSharedRef<FSceneViewFamilyContext> ViewFamily = FMovieGraphDeferredPass::CreateSceneViewFamily(InInitData);

	// DOF splats require engine DOF to be enabled
	const bool bDOFSplatsEnabled = GetEffectiveDOFSplatSize() > UE_KINDA_SMALL_NUMBER;
	ViewFamily->EngineShowFlags.SetDepthOfField(bDOFSplatsEnabled);

	// Motion blur is only enabled during injection pass when requested
	const bool bWantsMotionBlur = CachedDOFComponent.IsValid() && CachedDOFComponent->bEnableMotionBlur;
	const bool bMotionBlurEnabled = bWantsMotionBlur && !FMath::IsNearlyZero(CachedMotionBlurFraction);
	ViewFamily->EngineShowFlags.SetMotionBlur(bMotionBlurEnabled);

	// AA show flags: when bUseJitterAA is true (default), TAA show flags are disabled so the pass
	// uses its own jitter-based AA via aperture sampling. When false, TAA show flags are enabled
	// and actual TAA processing is controlled via AntiAliasingMethod in ApplyMovieGraphOverridesToSceneView.
	const bool bUseJitterAA = CachedDOFComponent.IsValid() ? CachedDOFComponent->bUseJitterAA : true;
	ViewFamily->EngineShowFlags.SetTemporalAA(!bUseJitterAA);
	ViewFamily->EngineShowFlags.SetAntiAliasing(!bUseJitterAA);
	
	// Add AccumulationDOF scene view extensions (equivalent to MRQ's AddViewExtensions)
	if (bInCapturePass && CaptureExtension.IsValid())
	{
		ViewFamily->ViewExtensions.Add(CaptureExtension.ToSharedRef());
	}

	if (bInInjectionPass && InjectionExtension.IsValid())
	{
		ViewFamily->ViewExtensions.Add(InjectionExtension.ToSharedRef());
	}

	return ViewFamily;
}

FVector2D FMovieGraphAccumulationDOFPass::ComputeProjectionMatrixJitter(const FVector2D& InDefaultJitter, const FIntPoint& InBackbufferResolution) const
{
	if (ApertureSampler && CurrentApertureSampleIndex >= 0)
	{
		return ApertureSampler->GetJitterForProjectionMatrix(CurrentApertureSampleIndex, InBackbufferResolution);
	}

	return FVector2D::ZeroVector;
}

FSceneViewInitOptions FMovieGraphAccumulationDOFPass::CreateViewInitOptions(
	const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo,
	FSceneViewFamilyContext* InViewFamily,
	FSceneViewStateInterface* InViewStateInterface) const
{
	FSceneViewInitOptions ViewInitOptions = FMovieGraphDeferredPass::CreateViewInitOptions(InCameraInfo, InViewFamily, InViewStateInterface);

	// Apply aperture sample camera offset in the lens plane
	if (ApertureSampler && CurrentApertureSampleIndex >= 0 && CurrentApertureSampleIndex < ApertureSampler->GetActualNumSamples())
	{
		const FVector2f ApertureOffset = ApertureSampler->GetApertureOffset(CurrentApertureSampleIndex);

		const FRotator CameraRotation = InCameraInfo.ViewInfo.Rotation;
		const FVector CameraRight = FRotationMatrix(CameraRotation).GetUnitAxis(EAxis::Y);
		const FVector CameraUp = FRotationMatrix(CameraRotation).GetUnitAxis(EAxis::Z);

		const FVector OffsetLocation = InCameraInfo.ViewInfo.Location + CameraRight * ApertureOffset.X + CameraUp * ApertureOffset.Y;

		ViewInitOptions.ViewOrigin = OffsetLocation;
		ViewInitOptions.ViewLocation = OffsetLocation;
	}

	return ViewInitOptions;
}

void FMovieGraphAccumulationDOFPass::CalculateProjectionMatrix(
	UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo,
	FSceneViewProjectionData& InOutProjectionData,
	const FIntPoint InBackbufferResolution,
	const FIntPoint InAccumulatorResolution) const
{
	if (ApertureSampler && CurrentApertureSampleIndex >= 0 && CurrentApertureSampleIndex < ApertureSampler->GetActualNumSamples())
	{
		FMatrix ProjectionMatrix;

		if (ApertureSampler->ComputeProjectionMatrix(CurrentApertureSampleIndex, CurrentFocusDistanceOverride, InOutCameraInfo.ViewInfo, ProjectionMatrix))
		{
			InOutProjectionData.ProjectionMatrix = ProjectionMatrix;

			// Reset PreviousViewTransform — with aperture sampling, keeping the center position
			// gives Lumen false camera movement causing what looks like reprojection drift.
			InOutCameraInfo.ViewInfo.PreviousViewTransform.Reset();
			return;
		}
	}

	// Fallback to default projection matrix calculation
	FMovieGraphDeferredPass::CalculateProjectionMatrix(InOutCameraInfo, InOutProjectionData, InBackbufferResolution, InAccumulatorResolution);
}

void FMovieGraphAccumulationDOFPass::ApplyMovieGraphOverridesToSceneView(
	TSharedRef<FSceneViewFamilyContext> InOutFamily,
	const UE::MovieGraph::Rendering::FViewFamilyInitData& InInitData,
	const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	FMovieGraphDeferredPass::ApplyMovieGraphOverridesToSceneView(InOutFamily, InInitData, InCameraInfo);

	FSceneView* View = const_cast<FSceneView*>(InOutFamily->Views[0]);

	const bool bUseJitterAA = CachedDOFComponent.IsValid() ? CachedDOFComponent->bUseJitterAA : true;
	const bool bWantsMotionBlur = CachedDOFComponent.IsValid() && CachedDOFComponent->bEnableMotionBlur;
	const bool bMotionBlurInjection = bInInjectionPass && bWantsMotionBlur && !FMath::IsNearlyZero(CachedMotionBlurFraction);

	// The temporal-upscaler injection path requires a temporal AA method to run, so use TAA when
	// bUseJitterAA is false or this is the motion-blur injection pass.
	const bool bCanUseTAA = (!bUseJitterAA || bMotionBlurInjection) && View->State != nullptr;
	View->AntiAliasingMethod = bCanUseTAA ? AAM_TemporalAA : AAM_None;

	// Disable motion blur for capture passes — only the injection pass should have motion blur
	if (!bInInjectionPass)
	{
		View->FinalPostProcessSettings.MotionBlurAmount = 0.0f;
		View->FinalPostProcessSettings.bOverride_MotionBlurAmount = true;
	}

	// During injection, disable engine's lateral chromatic aberration if we applied our own spectral version
	if (bInInjectionPass)
	{
		const bool bUseSpectralLateralCA = CachedDOFComponent.IsValid()
			? CachedDOFComponent->bSpectralLateralChromaticAberration
			: true;

		const bool bAppliedOurSpectralCA = bUseSpectralLateralCA && (CachedSceneFringeIntensity > UE_KINDA_SMALL_NUMBER);

		if (bAppliedOurSpectralCA)
		{
			View->FinalPostProcessSettings.SceneFringeIntensity = 0.0f;
		}
	}
}

FSceneViewStateInterface* FMovieGraphAccumulationDOFPass::GetSceneViewState(UMovieGraphImagePassBaseNode* ParentNodeThisFrame, int32_t TileX, int32_t TileY)
{
	// During injection pass, use the per-tile dedicated injection view state.
	if (bInInjectionPass)
	{
		FSceneViewStateReference* Ref = InjectionViewStates.Find(FIntPoint(TileX, TileY));
		if (ensureMsgf(Ref, TEXT("AccumulationDOF: missing per-tile injection view state for [%d,%d] (CurrentTileCount [%d,%d]). Falling back to base view state — temporal history will be wrong."),
				TileX, TileY, CurrentTileCount.X, CurrentTileCount.Y))
		{
			if (FSceneViewStateInterface* ViewStateInterface = Ref->GetReference())
			{
				return ViewStateInterface;
			}
		}
	}

	FSceneViewStateInterface* BaseState = FMovieGraphDeferredPass::GetSceneViewState(ParentNodeThisFrame, TileX, TileY);

	return BaseState;
}

void FMovieGraphAccumulationDOFPass::GetPreviewBannerMessages(TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const
{
	FMovieGraphPreviewBannerMessage& Message = OutMessages.AddDefaulted_GetRef();
	Message.Message = NSLOCTEXT("AccumulationDOF", "PreviewBanner_AccumulationDOF", "Accumulation DOF");
	Message.TextColor = FSlateColor(EStyleColor::AccentOrange).GetSpecifiedColor();
}

void FMovieGraphAccumulationDOFPass::RenderSamples(
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData,
	const FResolutionAndCameraInfo& InResolutionInfo,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams)
{
	const int32 ActualNumSamples = ApertureSampler->GetActualNumSamples();

	// Note: ResetAccumulation is called per-tile in Render() before RenderSamples is invoked.

	// Check for axial chromatic aberration
	const float AxialCAIntensity = CachedDOFComponent.IsValid()
		? CachedDOFComponent->AxialChromaticAberrationIntensity
		: 0.0f;

	const int32 EffectiveBands = ApertureSampler->GetEffectiveBands();
	const bool bHasAxialCA = (AxialCAIntensity > UE_KINDA_SMALL_NUMBER) && (EffectiveBands > 1);

	if (bHasAxialCA)
	{
		// Axial CA: render each sample with varying focus distances per spectral band
		const FVector3f TotalSpectralWeight = ApertureSampler->GetTotalSpectralWeightPerChannel();

		const FVector3f InvTotal(
			(TotalSpectralWeight.X > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeight.X) : 0.0f,
			(TotalSpectralWeight.Y > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeight.Y) : 0.0f,
			(TotalSpectralWeight.Z > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeight.Z) : 0.0f
		);

		// Get nominal focus distance for spectral calculations
		const float NominalFocusDistanceCm = CachedCineCameraComponent.IsValid()
			? CachedCineCameraComponent->CurrentFocusDistance
			: 1000.0f;

		int32 IterationCount = 0;

		for (int32 SampleIndex = 0; SampleIndex < ActualNumSamples; ++SampleIndex)
		{
			for (int32 Band = 0; Band < EffectiveBands; ++Band)
			{
				CurrentApertureSampleIndex = SampleIndex;

				// Compute spectral parameters for this band
				const float NormalizedSpectralPosition = (EffectiveBands == 1)
					? 0.5f
					: static_cast<float>(Band) / static_cast<float>(EffectiveBands - 1);

				CurrentFocusDistanceOverride = ApertureSampler->ComputeSpectralFocusDistance(
					NominalFocusDistanceCm, NormalizedSpectralPosition);

				const FVector3f RawWeight = ApertureSampler->ComputeSpectralWeight(NormalizedSpectralPosition);
				CurrentAxialCAWeight = RawWeight * InvTotal;

				RenderSingleApertureSample(SampleIndex, InFrameTraversalContext, InTimeData, InResolutionInfo, InRenderTargetInitParams);

				IterationCount++;

				// Release memory periodically
				if ((IterationCount % AccumulationDOFUtils::RenderingMemoryFlushBatchSize) == 0)
				{
					AccumulationDOFUtils::ReleaseRenderingMemory();
				}
			}
		}
	}
	else
	{
		// No axial CA: standard rendering with uniform spectral weight
		CurrentFocusDistanceOverride = -1.0f;
		CurrentAxialCAWeight = FVector3f(1.0f);

		for (int32 SampleIndex = 0; SampleIndex < ActualNumSamples; ++SampleIndex)
		{
			CurrentApertureSampleIndex = SampleIndex;
			RenderSingleApertureSample(SampleIndex, InFrameTraversalContext, InTimeData, InResolutionInfo, InRenderTargetInitParams);
		}
	}

	// Reset state after all samples rendered
	CurrentApertureSampleIndex = -1;
	CurrentFocusDistanceOverride = -1.0f;
	CurrentAxialCAWeight = FVector3f(1.0f);
	CaptureExtension.Reset();

	// Update camera state with captured SceneFringeIntensity for lateral CA
	ApertureSampler->UpdateCameraState(BuildCameraState());

	// Finalize accumulation
	ApertureSampler->FinalizeAccumulation();

	// Render injection pass and submit to MRG
	RenderInjectionAndSubmit(InFrameTraversalContext, InTimeData, InResolutionInfo, InRenderTargetInitParams);
}

void FMovieGraphAccumulationDOFPass::RenderSingleApertureSample(
	int32 SampleIndex,
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData,
	const FResolutionAndCameraInfo& InResolutionInfo,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams)
{
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	UMovieGraphImagePassBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);
	const FIntPoint BackbufferResolution = InResolutionInfo.BackbufferResolution;
	const FIntPoint AccumulatorResolution = InResolutionInfo.AccumulatorResolution;

	// Create SVE to capture scene color
	UTextureRenderTarget2D* SampleRT = ApertureSampler->GetSampleRT();

	if (SampleRT)
	{
		FAccumulationDOFSVESettings SVESettings = FAccumulationDOFSVESettings::FromCineCameraComponent(CachedCineCameraComponent.Get());

		CaptureExtension = MakeShared<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe>(
			SampleRT,
			EAccumulationDOFSVEMode::Capture,
			SVESettings
		);

		// Apply DOF splats if size is > 0
		const float EffectiveDOFSplatSize = GetEffectiveDOFSplatSize();

		if (EffectiveDOFSplatSize > UE_KINDA_SMALL_NUMBER)
		{
			const AccumulationDOFUtils::FCameraParams& CamParams = ApertureSampler->GetConfig().NumSamples > 0 ?
				AccumulationDOFUtils::ExtractCameraParams(CachedCineCameraComponent.Get()) :
				AccumulationDOFUtils::FCameraParams();

			const float DOFSplatsFStop = AccumulationDOFMath::ComputeDOFSplatsFStop(
				CamParams.ApertureRadiusCm,
				CamParams.FocalLengthMm,
				EffectiveDOFSplatSize
			);

			const float SafeSqueezeFactor = FMath::Max(CamParams.SqueezeFactor, UE_KINDA_SMALL_NUMBER);
			if (DOFSplatsFStop > UE_KINDA_SMALL_NUMBER)
			{
				const float SensorWidthMm = CamParams.SensorWidthMm / SafeSqueezeFactor;
				const bool bForceNeutralBokeh = true;

				CaptureExtension->SetDOFSplatsSettings(DOFSplatsFStop, CamParams.FocusDistanceCm, SensorWidthMm, CamParams.SqueezeFactor, bForceNeutralBokeh);
			}
		}
	}
	else
	{
		CaptureExtension.Reset();
	}

	// Get render target
	UTextureRenderTarget2D* RenderTarget = GraphRenderer->GetOrCreateViewRenderTarget(InRenderTargetInitParams, RenderDataIdentifier);
	FRenderTarget* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
	check(RenderTargetResource);

	// Build camera info
	UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();
	CameraInfo.bAllowCameraAspectRatio = true;
	CameraInfo.TilingParams.TileSize = InResolutionInfo.TileSize;
	CameraInfo.TilingParams.TileCount = CurrentTileCount;
	CameraInfo.TilingParams.TileIndexes = CurrentTileIndex;
	CameraInfo.TilingParams.OverlapPad = CurrentTileOverlappedPad;
	CameraInfo.SamplingParams.TemporalSampleIndex = InTimeData.TemporalSampleIndex;
	CameraInfo.SamplingParams.TemporalSampleCount = InTimeData.TemporalSampleCount;
	CameraInfo.SamplingParams.SpatialSampleIndex = 0;
	CameraInfo.SamplingParams.SpatialSampleCount = 1;
	CameraInfo.ProjectionMatrixJitterAmount = ComputeProjectionMatrixJitter(FVector2D::ZeroVector, BackbufferResolution);
	CameraInfo.bUseCameraManagerPostProcess = !InResolutionInfo.bRenderAllCameras;
	{
		const FIntPoint OutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(InTimeData.EvaluatedConfig.Get(), InResolutionInfo.CameraAspectRatio);
		ReapplyOverscanPreservingEngineScaling(CameraInfo, InResolutionInfo.OverscanFraction, InResolutionInfo.AccumulatorResolution, OutputResolution);
	}

	// Build view family init data
	const int32 FrameIndex = InTimeData.RenderedFrameNumber;
	const ESceneCaptureSource SceneCaptureSource = ParentNodeThisFrame->GetDisableToneCurve() ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;

	UE::MovieGraph::Rendering::FViewFamilyInitData ViewFamilyInitData;
	ViewFamilyInitData.RenderTarget = RenderTargetResource;
	ViewFamilyInitData.World = GraphRenderer->GetWorld();
	ViewFamilyInitData.TimeData = InTimeData;
	ViewFamilyInitData.SceneCaptureSource = SceneCaptureSource;
	ViewFamilyInitData.bWorldIsPaused = false; // Will be set below based on TemporalHistoryMode
	ViewFamilyInitData.FrameIndex = FrameIndex;
	ViewFamilyInitData.AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
	ViewFamilyInitData.ShowFlags = ParentNodeThisFrame->GetShowFlags();
	ViewFamilyInitData.ViewModeIndex = ParentNodeThisFrame->GetViewModeIndex();
	ViewFamilyInitData.ProjectionMode = CameraInfo.ViewInfo.ProjectionMode;

	bInCapturePass = true;
	FCreateViewResult ViewResult = CreateConfiguredView(CameraInfo, ViewFamilyInitData, BackbufferResolution, AccumulatorResolution, ParentNodeThisFrame, CurrentTileIndex.X, CurrentTileIndex.Y);
	bInCapturePass = false;

	TSharedRef<FSceneViewFamilyContext> ViewFamily = ViewResult.ViewFamily;
	FSceneView* View = ViewResult.View;

	if (!View)
	{
		UE_LOGF(LogAccumulationDOFMRG, Error, "Failed to create view for aperture sample %d", SampleIndex);
		CaptureExtension.Reset();
		return;
	}

	// Modify OverrideFrameIndexValue — combine with tile index and aperture sample index for uniqueness so
	// each tile gets a distinct TAA/jitter pseudo-random sequence.
	const int32 ActualNumSamples = ApertureSampler->GetActualNumSamples();
	if (View->OverrideFrameIndexValue.IsSet())
	{
		const int32 TotalTiles = FMath::Max(1, CurrentTileCount.X * CurrentTileCount.Y);
		const int32 TileLinearIndex = CurrentTileIndex.Y * CurrentTileCount.X + CurrentTileIndex.X;
		const int64 BaseFrameIndex = View->OverrideFrameIndexValue.GetValue();
		const int64 CombinedIndex = BaseFrameIndex * static_cast<int64>(ActualNumSamples) * static_cast<int64>(TotalTiles)
			+ static_cast<int64>(TileLinearIndex) * static_cast<int64>(ActualNumSamples)
			+ static_cast<int64>(SampleIndex);
		View->OverrideFrameIndexValue = static_cast<int32>(CombinedIndex & 0x7FFFFFFF);
	}

	// Control when temporal history updates with bWorldIsPaused
	const bool bIsFirstApertureSample = (SampleIndex == 0);
	const bool bIsLastApertureSample = (SampleIndex == ActualNumSamples - 1);

	const ETemporalHistoryMode HistoryMode =
		CachedDOFComponent.IsValid() ? CachedDOFComponent->TemporalHistoryMode : ETemporalHistoryMode::LastSampleOnly;

	switch (HistoryMode)
	{
	case ETemporalHistoryMode::AllSamplesUpdate:
		ViewFamily->bWorldIsPaused = false;
		break;
	case ETemporalHistoryMode::FirstSampleOnly:
		ViewFamily->bWorldIsPaused = !bIsFirstApertureSample;
		break;
	case ETemporalHistoryMode::LastSampleOnly:
		ViewFamily->bWorldIsPaused = !bIsLastApertureSample;
		break;
	case ETemporalHistoryMode::NoSamplesUpdate:
		ViewFamily->bWorldIsPaused = true;
		break;
	default:
		checkNoEntry();
		break;
	}

	// Render
	FCanvas Canvas = FCanvas(RenderTargetResource, nullptr, GraphRenderer->GetWorld(), GraphRenderer->GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
	GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

	// Transition the pooled view RT from RTV to SRV so AccumulateExternalSample can sample
	// it. Matches FMovieGraphDeferredPass::Render().
	ENQUEUE_RENDER_COMMAND(TransitionApertureSampleRTToSRV)(
		[RenderTargetResource](FRHICommandListImmediate& RHICmdList) mutable
		{
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
		});

	// Note: page-to-system-memory backup of the capture view state is handled once per tile in Render(),
	// not per aperture sample.

	// Capture SceneFringeIntensity from first sample
	if (SampleIndex == 0 && CaptureExtension.IsValid())
	{
		const float CapturedValue = CaptureExtension->GetCapturedSceneFringeIntensity();
		if (FMath::Abs(CapturedValue - CachedSceneFringeIntensity) > UE_KINDA_SMALL_NUMBER)
		{
			CachedSceneFringeIntensity = CapturedValue;
		}
	}

	// Accumulate with spectral weight (set per band for axial CA)
	const FVector2f ApertureOffset = ApertureSampler->GetApertureOffset(SampleIndex);
	ApertureSampler->AccumulateExternalSample(SampleIndex, ApertureOffset, CurrentAxialCAWeight);

	// Release memory periodically
	if (((SampleIndex + 1) % AccumulationDOFUtils::RenderingMemoryFlushBatchSize) == 0)
	{
		AccumulationDOFUtils::ReleaseRenderingMemory();
	}

	CaptureExtension.Reset();
}

void FMovieGraphAccumulationDOFPass::RenderInjectionAndSubmit(
	const FMovieGraphTraversalContext& InFrameTraversalContext,
	const FMovieGraphTimeStepData& InTimeData,
	const FResolutionAndCameraInfo& InResolutionInfo,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams)
{
	TObjectPtr<UMovieGraphDefaultRenderer> GraphRenderer = GetRenderer().Get();
	if (!GraphRenderer)
	{
		return;
	}

	UMovieGraphImagePassBaseNode* ParentNodeThisFrame = GetParentNode(InTimeData.EvaluatedConfig);
	const FIntPoint BackbufferResolution = InResolutionInfo.BackbufferResolution;
	const FIntPoint AccumulatorResolution = InResolutionInfo.AccumulatorResolution;

	// Get our accumulated result for injection
	UTextureRenderTarget2D* NormalizedRT = ApertureSampler->GetAccumulatedResult();

	if (!NormalizedRT)
	{
		UE_LOGF(LogAccumulationDOFMRG, Error, "No normalized render target available for injection pass");
		return;
	}

	// Determine motion blur setting
	const bool bWantsMotionBlur = CachedDOFComponent.IsValid() && CachedDOFComponent->bEnableMotionBlur;
	const bool bMotionBlurEnabled = bWantsMotionBlur && !FMath::IsNearlyZero(CachedMotionBlurFraction);

	// Create injection extension
	{
		const EAccumulationDOFSVEMode InjectionMode = bMotionBlurEnabled
			? EAccumulationDOFSVEMode::InjectViaTemporalUpscaler
			: EAccumulationDOFSVEMode::Inject;

		InjectionExtension = MakeShared<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe>(
			NormalizedRT,
			InjectionMode
		);

		const bool bUseSpectralLateralCA =
			CachedDOFComponent.IsValid() ?
			CachedDOFComponent->bSpectralLateralChromaticAberration
			: true;

		const bool bAppliedOurSpectralCA = bUseSpectralLateralCA && (CachedSceneFringeIntensity > UE_KINDA_SMALL_NUMBER);
		InjectionExtension->SetAllowSceneFringe(!bAppliedOurSpectralCA);
	}

	// Get render target
	UTextureRenderTarget2D* RenderTarget = GraphRenderer->GetOrCreateViewRenderTarget(InRenderTargetInitParams, RenderDataIdentifier);
	FRenderTarget* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
	check(RenderTargetResource);

	// Build camera info for injection
	UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();
	CameraInfo.bAllowCameraAspectRatio = true;
	CameraInfo.TilingParams.TileSize = InResolutionInfo.TileSize;
	CameraInfo.TilingParams.TileCount = CurrentTileCount;
	CameraInfo.TilingParams.TileIndexes = CurrentTileIndex;
	CameraInfo.TilingParams.OverlapPad = CurrentTileOverlappedPad;
	CameraInfo.SamplingParams.TemporalSampleIndex = InTimeData.TemporalSampleIndex;
	CameraInfo.SamplingParams.TemporalSampleCount = InTimeData.TemporalSampleCount;
	CameraInfo.SamplingParams.SpatialSampleIndex = 0;
	CameraInfo.SamplingParams.SpatialSampleCount = 1;
	CameraInfo.ProjectionMatrixJitterAmount = FVector2D::ZeroVector;
	CameraInfo.bUseCameraManagerPostProcess = !InResolutionInfo.bRenderAllCameras;
	{
		const FIntPoint OutputResolution = UMovieGraphBlueprintLibrary::GetDesiredOutputResolution(InTimeData.EvaluatedConfig.Get(), InResolutionInfo.CameraAspectRatio);
		ReapplyOverscanPreservingEngineScaling(CameraInfo, InResolutionInfo.OverscanFraction, InResolutionInfo.AccumulatorResolution, OutputResolution);
	}

	// Build view family init data
	const ESceneCaptureSource SceneCaptureSource = ParentNodeThisFrame->GetDisableToneCurve() ? ESceneCaptureSource::SCS_FinalColorHDR : ESceneCaptureSource::SCS_FinalToneCurveHDR;

	UE::MovieGraph::Rendering::FViewFamilyInitData ViewFamilyInitData;
	ViewFamilyInitData.RenderTarget = RenderTargetResource;
	ViewFamilyInitData.World = GraphRenderer->GetWorld();
	ViewFamilyInitData.TimeData = InTimeData;
	ViewFamilyInitData.SceneCaptureSource = SceneCaptureSource;
	ViewFamilyInitData.bWorldIsPaused = false;
	ViewFamilyInitData.FrameIndex = InTimeData.RenderedFrameNumber;
	ViewFamilyInitData.AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
	ViewFamilyInitData.ShowFlags = ParentNodeThisFrame->GetShowFlags();
	ViewFamilyInitData.ViewModeIndex = ParentNodeThisFrame->GetViewModeIndex();
	ViewFamilyInitData.ProjectionMode = CameraInfo.ViewInfo.ProjectionMode;

	// Page-to-system-memory: explicitly restore this tile's injection view state. The base's auto-restore
	// in CreateConfiguredView is suppressed (bAutoRestoreViewStateMirror=false in Setup), so we drive it
	// here. Symmetric with the backup added after BeginRenderingViewFamily below.
	if (ParentNodeThisFrame->GetEnableHistoryPerTile() && ParentNodeThisFrame->GetEnablePageToSystemMemory())
	{
		if (FSceneViewStateReference* InjectionRef = InjectionViewStates.Find(CurrentTileIndex))
		{
			if (FSceneViewStateInterface* InjectionViewState = InjectionRef->GetReference())
			{
				InjectionViewState->SystemMemoryMirrorRestore(SystemMemoryMirror.Get());
			}
		}
	}

	bInInjectionPass = true;
	FCreateViewResult ViewResult = CreateConfiguredView(CameraInfo, ViewFamilyInitData, BackbufferResolution, AccumulatorResolution, ParentNodeThisFrame, CurrentTileIndex.X, CurrentTileIndex.Y);
	bInInjectionPass = false;

	TSharedRef<FSceneViewFamilyContext> ViewFamily = ViewResult.ViewFamily;
	FSceneView* View = ViewResult.View;

	if (!View)
	{
		InjectionExtension.Reset();
		UE_LOGF(LogAccumulationDOFMRG, Error, "Failed to create view for injection pass");
		return;
	}

	// On the first injection render for each tile, its per-tile injection view state has no temporal
	// history (warmup only renders through the main SceneViewStates). Set bCameraCut to signal the
	// renderer to initialize temporal effects (eye adaptation, etc.) from scratch instead of reading
	// uninitialized history, which would produce a black frame.
	if (!InjectionStatesInitialized.Contains(CurrentTileIndex))
	{
		View->bCameraCut = true;
		InjectionStatesInitialized.Add(CurrentTileIndex);
	}

	// Render injection
	FCanvas Canvas = FCanvas(RenderTargetResource, nullptr, GraphRenderer->GetWorld(), GraphRenderer->GetWorld()->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
	GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

	// Transition the render target from RTV to SRV so the surface reader can safely
	// read back the injection result. Without this, the GPU may read stale cache data
	// on RHI backends with explicit state tracking. This matches the transition done
	// by FMovieGraphDeferredPass::Render after BeginRenderingViewFamily.
	ENQUEUE_RENDER_COMMAND(TransitionInjectionRTToSRV)(
		[RenderTargetResource](FRHICommandListImmediate& RHICmdList) mutable
		{
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
		});

	// After submission, if we're paging to system memory, mark the resources for download into system memory.
	if (ParentNodeThisFrame->GetEnableHistoryPerTile() && ParentNodeThisFrame->GetEnablePageToSystemMemory())
	{
		if (FSceneViewStateReference* InjectionRef = InjectionViewStates.Find(CurrentTileIndex))
		{
			if (FSceneViewStateInterface* InjectionViewState = InjectionRef->GetReference())
			{
				InjectionViewState->SystemMemoryMirrorBackup(SystemMemoryMirror.Get());
			}
		}
	}

	InjectionExtension.Reset();

	// Build sample state and submit — DOF calls PostRendererSubmission directly (no FIFO queue delay)
	UE::MovieGraph::FMovieGraphSampleState SampleState;
	{
		FMovieGraphTraversalContext UpdatedTraversalContext = InFrameTraversalContext;
		UpdatedTraversalContext.Time = InTimeData;
		UpdatedTraversalContext.Time.SpatialSampleIndex = 0;
		UpdatedTraversalContext.Time.SpatialSampleCount = 1;
		UpdatedTraversalContext.RenderDataIdentifier = RenderDataIdentifier;

		const bool bHasTiles = (CurrentTileCount.X * CurrentTileCount.Y) > 1;
		const bool bIsLastTile = (CurrentTileIndex == CurrentTileCount - FIntPoint(1, 1));
		const FIntPoint OverlappedOffset = FIntPoint(
			CurrentTileIndex.X * InResolutionInfo.TileSize.X - CurrentTileOverlappedPad.X,
			CurrentTileIndex.Y * InResolutionInfo.TileSize.Y - CurrentTileOverlappedPad.Y);

		SampleState.TraversalContext = MoveTemp(UpdatedTraversalContext);
		SampleState.OverscannedResolution = AccumulatorResolution;
		SampleState.UnpaddedTileSize = InResolutionInfo.TileSize;
		SampleState.BackbufferResolution = BackbufferResolution;
		SampleState.AccumulatorResolution = AccumulatorResolution;
		SampleState.bWriteSampleToDisk = false;
		SampleState.bRequiresAccumulator = InTimeData.bRequiresAccumulator || bHasTiles;
		SampleState.bFetchFromAccumulator = InTimeData.bIsLastTemporalSampleForFrame && bIsLastTile;
		SampleState.OverlappedPad = CurrentTileOverlappedPad;
		SampleState.OverlappedOffset = OverlappedOffset;
		SampleState.OverlappedSubpixelShift = FVector2D(0.5, 0.5); // ADOF does no spatial jitter
		SampleState.OverscanFraction = CameraInfo.ViewInfo.GetOverscan();
		SampleState.CropRectangle = InResolutionInfo.AccumulatorResolutionCropRect;
		SampleState.bAllowOCIO = ParentNodeThisFrame->GetAllowOCIO();
		SampleState.bForceLosslessCompression = ParentNodeThisFrame->GetForceLosslessCompression();
		SampleState.bAllowsCompositing = ParentNodeThisFrame->GetAllowsCompositing();
		SampleState.bIsBeautyPass = true;
		SampleState.SceneCaptureSource = SceneCaptureSource;
		SampleState.CompositingSortOrder = 10;
		SampleState.RenderLayerIndex = LayerData.LayerIndex;
	}

	ApplyMovieGraphOverridesToSampleState(SampleState);
	WritePerViewMetadata(SampleState.AdditionalFileMetadata, *View, ViewResult.SceneViewInitOptions, CameraInfo, RenderDataIdentifier);

	// Accumulation DOF metadata
	{
		const FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(RenderDataIdentifier);
		const int32 ActualNumSamples = ApertureSampler ? ApertureSampler->GetActualNumSamples() : 0;
		SampleState.AdditionalFileMetadata.Add(
			FString::Printf(TEXT("%s/accumulationDof/numSamples"), *MetadataPrefix), FString::FromInt(ActualNumSamples));
		SampleState.AdditionalFileMetadata.Add(
			FString::Printf(TEXT("%s/accumulationDof/splatSize"), *MetadataPrefix), FString::SanitizeFloat(GetEffectiveDOFSplatSize()));
	}

	PostRendererSubmission(SampleState, InRenderTargetInitParams, Canvas, CameraInfo);

	// Flush render commands so the surface reader captures this tile's injection result before the next tile's
	// (or next frame's) aperture samples reuse the pooled view RT. GetOrCreateViewRenderTarget returns the same
	// pooled RT instance keyed by RenderDataIdentifier, so without this flush the next tile's capture would
	// overwrite pixels we haven't read back yet. This serializes game/render threads briefly, but is negligible
	// compared to the cost of rendering N aperture samples per tile.
	FlushRenderingCommands();
}

AccumulationDOF::FApertureSamplerConfig FMovieGraphAccumulationDOFPass::BuildSamplerConfig(UMovieGraphEvaluatedConfig* InEvaluatedConfig) const
{
	AccumulationDOF::FApertureSamplerConfig Config;

	Config.World = GetRenderer()->GetWorld();

	// Resolution & overscan: BackbufferResolution is the per-tile resolution with overscan
	const FResolutionAndCameraInfo ResolutionAndCameraInfo = GetResolutionAndCameraInfo(InEvaluatedConfig);
	Config.Resolution = ResolutionAndCameraInfo.BackbufferResolution;
	Config.OverscanFraction = ResolutionAndCameraInfo.OverscanFraction;

	// Sampling settings
	Config.Mode = AccumulationDOF::EApertureSamplerMode::OneShot;

	// Get settings from component (should always be valid if there's an Accumulation DOF pass active)
	if (CachedDOFComponent.IsValid())
	{
		Config.NumSamples                          = CachedDOFComponent->NumSamples;
		Config.SamplingPattern                     = CachedDOFComponent->SamplingPattern;
		Config.DOFSplatSize	                       = CachedDOFComponent->DOFSplatSize;
		Config.AxialChromaticAberrationIntensity   = CachedDOFComponent->AxialChromaticAberrationIntensity;
		Config.AxialChromaticAberrationNumBands    = CachedDOFComponent->AxialChromaticAberrationNumBands;
		Config.bSpectralLateralChromaticAberration = CachedDOFComponent->bSpectralLateralChromaticAberration;
		Config.SphericalAberration                 = CachedDOFComponent->SphericalAberration;
		Config.ComaAberration                      = CachedDOFComponent->ComaAberration;
		Config.BokehTexture                        = CachedDOFComponent->BokehTexture;
		Config.bEnableBokehTexture                 = CachedDOFComponent->bEnableBokehTexture;
		Config.WeightChannel                       = CachedDOFComponent->WeightChannel;
		Config.TintStrength                        = CachedDOFComponent->TintStrength;
		Config.BokehEdgeSoftness                   = CachedDOFComponent->BokehEdgeSoftness;
		Config.TemporalHistoryMode                 = CachedDOFComponent->TemporalHistoryMode;
		Config.bUseJitterAA                        = CachedDOFComponent->bUseJitterAA;
	}

	Config.ViewModeIndex = VMI_Lit;

	return Config;
}

AccumulationDOF::FApertureSamplerCameraState FMovieGraphAccumulationDOFPass::BuildCameraState() const
{
	AccumulationDOF::FApertureSamplerCameraState State;

	State.CineCameraComponent = CachedCineCameraComponent;

	if (CachedCineCameraComponent.IsValid())
	{
		State.CameraLocation = CachedCineCameraComponent->GetComponentLocation();
		State.CameraRotation = CachedCineCameraComponent->GetComponentRotation();
		State.bInitialized = true;
	}

	State.SceneFringeIntensity = CachedSceneFringeIntensity;

	return State;
}

void FMovieGraphAccumulationDOFPass::InitializeSampler(UMovieGraphEvaluatedConfig* InEvaluatedConfig)
{
	// Find camera first
	CachedCineCameraComponent = FindCineCameraComponent();
	FindAccumulationDOFComponent();

	// Build config and camera state
	const AccumulationDOF::FApertureSamplerConfig Config = BuildSamplerConfig(InEvaluatedConfig);
	const AccumulationDOF::FApertureSamplerCameraState CameraState = BuildCameraState();

	// Create and initialize sampler
	ApertureSampler = NewObject<UApertureSampler>(WeakGraphRenderer.Get());

	if (!ApertureSampler->Initialize(Config, CameraState))
	{
		UE_LOGF(LogAccumulationDOFMRG, Error, "Failed to initialize ApertureSampler");

		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}
	else
	{
		CachedActualNumSamples = ApertureSampler->GetActualNumSamples();
		CachedEffectiveDOFSplatSize = Config.DOFSplatSize;
	}
}

UCineCameraComponent* FMovieGraphAccumulationDOFPass::FindCineCameraComponent() const
{
	const UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();

	const ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(CameraInfo.ViewActor);
	if (!CineCameraActor)
	{
		return nullptr;
	}

	return CineCameraActor->GetCineCameraComponent();
}

UAccumulationDOFComponent* FMovieGraphAccumulationDOFPass::FindAccumulationDOFComponent()
{
	if (CachedDOFComponent.IsValid())
	{
		return CachedDOFComponent.Get();
	}

	const UE::MovieGraph::DefaultRenderer::FCameraInfo CameraInfo = GetRenderCameraInfo();

	const ACineCameraActor* CineCameraActor = Cast<ACineCameraActor>(CameraInfo.ViewActor);
	if (!CineCameraActor)
	{
		UE_LOGF(LogAccumulationDOFMRG, Warning, "FindAccumulationDOFComponent: No CineCameraActor available");
		return nullptr;
	}

	UAccumulationDOFComponent* DOFComponent = CineCameraActor->FindComponentByClass<UAccumulationDOFComponent>();
	if (DOFComponent)
	{
		CachedDOFComponent = DOFComponent;
	}

	return DOFComponent;
}

float FMovieGraphAccumulationDOFPass::GetEffectiveDOFSplatSize() const
{
	if (CachedDOFComponent.IsValid())
	{
		return CachedDOFComponent->DOFSplatSize;
	}

	// Default
	return 0.125f;
}
