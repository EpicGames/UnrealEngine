// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFSceneViewExtension.h"

#include "AccumulationDOFLog.h"
#include "AccumulationDOFShaders.h"
#include "AccumulationDOFTemporalUpscaler.h"
#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"

#include "Engine/TextureRenderTarget2D.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "ScreenPass.h"
#include "TextureResource.h"


FAccumulationDOFSVESettings FAccumulationDOFSVESettings::FromCineCameraComponent(UCineCameraComponent* CineCamera)
{
	FAccumulationDOFSVESettings OutSettings;
	OutSettings.CineCameraComponent = CineCamera;

	return OutSettings;
}

FAccumulationDOFSceneViewExtension::FAccumulationDOFSceneViewExtension(
	UTextureRenderTarget2D* InRenderTarget,
	EAccumulationDOFSVEMode InMode,
	const FAccumulationDOFSVESettings& InSettings)
	: Mode(InMode)
	, RenderTarget(InRenderTarget)
	, Settings(InSettings)
{
}

void FAccumulationDOFSceneViewExtension::SetDOFSplatsSettings(float InFStop, float InFocusDistanceCm, float InSensorWidthMm, float InSqueezeFactor, bool bInForceNeutralBokeh)
{
	DOFSplatsFStop              = InFStop;
	DOFSplatsFocusDistanceCm    = InFocusDistanceCm;
	DOFSplatsSensorWidthMm      = InSensorWidthMm;
	DOFSplatsSqueezeFactor      = InSqueezeFactor;
	bDOFSplatsForceNeutralBokeh = bInForceNeutralBokeh;
}

void FAccumulationDOFSceneViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	// Apply camera post-process settings if we have a CineCameraComponent

	UCineCameraComponent* CineCamera = Settings.CineCameraComponent.Get();

	if (CineCamera)
	{
		FMinimalViewInfo DesiredView;
		CineCamera->GetCameraView(0.0f, DesiredView);

		InView.OverridePostProcessSettings(DesiredView.PostProcessSettings, DesiredView.PostProcessBlendWeight);
	}

	// Post-process settings overrides depend on whether we are capturing the sample or injecting the accumulation.

	if (Mode == EAccumulationDOFSVEMode::Inject || Mode == EAccumulationDOFSVEMode::InjectViaTemporalUpscaler)
	{
		// Disable engine DOF
		const_cast<FSceneViewFamily*>(InView.Family)->EngineShowFlags.SetDepthOfField(false);

		// Disable engine lateral chromatic aberration unless explicitly allowed.
		if (!bAllowSceneFringe)
		{
			InView.FinalPostProcessSettings.SceneFringeIntensity = 0.0f;
		}

		// For temporal upscaler mode, configure the view for the third-party upscaler path
		if (Mode == EAccumulationDOFSVEMode::InjectViaTemporalUpscaler)
		{
			// Set the view to use temporal AA method - required for our temporal upscaler injection path
			InView.AntiAliasingMethod = EAntiAliasingMethod::AAM_TemporalAA;
			InView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
		}
	}
	else
	{
		// In Capture mode, store SceneFringe for later and apply DOFSplats settings if enabled

		CapturedSceneFringeIntensity = InView.FinalPostProcessSettings.SceneFringeIntensity;

		InView.FinalPostProcessSettings.SceneFringeIntensity = 0.0f;

		if (DOFSplatsFStop > UE_KINDA_SMALL_NUMBER)
		{
			// Compensate for f-stop override when using manual exposure with physical camera settings

			const float CameraFStop = InView.FinalPostProcessSettings.DepthOfFieldFstop;
			const bool bManualExposure = (InView.FinalPostProcessSettings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual);

			if (bManualExposure && InView.FinalPostProcessSettings.AutoExposureApplyPhysicalCameraExposure && CameraFStop > UE_KINDA_SMALL_NUMBER)
			{
				const float ExposureCompensation = 2.0f * FMath::Log2(DOFSplatsFStop / CameraFStop);
				InView.FinalPostProcessSettings.AutoExposureBias += ExposureCompensation;
			}

			InView.FinalPostProcessSettings.DepthOfFieldFstop         = DOFSplatsFStop;
			InView.FinalPostProcessSettings.DepthOfFieldFocalDistance = DOFSplatsFocusDistanceCm;
			InView.FinalPostProcessSettings.DepthOfFieldSensorWidth   = DOFSplatsSensorWidthMm;
			InView.FinalPostProcessSettings.DepthOfFieldSqueezeFactor = DOFSplatsSqueezeFactor;

			// Force neutral bokeh shape for DOFSplats to ensure even fill
			if (bDOFSplatsForceNeutralBokeh)
			{
				InView.FinalPostProcessSettings.DepthOfFieldPetzvalBokeh        = 0.0f;
				InView.FinalPostProcessSettings.DepthOfFieldPetzvalBokehFalloff = 0.0f;
				InView.FinalPostProcessSettings.DepthOfFieldBladeCount          = 16;   // Max blades for near-circular (@todo use MinFStop instead?)
				InView.FinalPostProcessSettings.DepthOfFieldBarrelRadius        = 0.0f; // Disable cat's eye
				InView.FinalPostProcessSettings.DepthOfFieldBarrelLength        = 0.0f;
			}
		}
	}
}

void FAccumulationDOFSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	// For InjectViaTemporalUpscaler mode, set up the temporal upscaler here.
	// Note: Can't be done earlier than BeginRenderViewFamily because there's a check.

	if (Mode == EAccumulationDOFSVEMode::InjectViaTemporalUpscaler && RenderTarget.IsValid())
	{
		TRefCountPtr<FRHITexture> RHITexture;
		if (FTextureRenderTargetResource* Resource = RenderTarget->GameThread_GetRenderTargetResource())
		{
			RHITexture = Resource->GetRenderTargetTexture();
		}

		FAccumulationDOFTemporalUpscaler* Upscaler = new FAccumulationDOFTemporalUpscaler(
			RHITexture,
			bDrawProgressBar ? ProgressBarFraction : -1.0f
		);

		InViewFamily.SetTemporalUpscalerInterface(Upscaler);
	}
}

void FAccumulationDOFSceneViewExtension::SubscribeToPostProcessingPass(
	EPostProcessingPass Pass,
	const FSceneView& View,
	FAfterPassCallbackDelegateArray& InOutPassCallbacks,
	bool bIsPassEnabled)
{
	if (Mode == EAccumulationDOFSVEMode::InjectViaTemporalUpscaler)
	{
		return;
	}

	if (Pass == EPostProcessingPass::MotionBlur)
	{
		InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(
			this, &FAccumulationDOFSceneViewExtension::ProcessAtMotionBlurPass_RenderThread)
		);
	}
}

FScreenPassTexture FAccumulationDOFSceneViewExtension::ProcessAtMotionBlurPass_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FPostProcessMaterialInputs& InOutInputs)
{
	UTextureRenderTarget2D* RenderTargetObj = RenderTarget.Get();
	if (!RenderTargetObj)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "AccumulationDOFSVE: No render target available");
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	FTextureRenderTargetResource* TextureResource = RenderTargetObj->GetRenderTargetResource();
	if (!TextureResource)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "AccumulationDOFSVE: No texture resource available");
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	// Get the input scene color
	const FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(
		GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor)
	);

	if (!SceneColor.IsValid())
	{
		UE_LOGF(LogAccumulationDOF, Warning, "AccumulationDOFSVE: Invalid scene color input");
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	FRHITexture* RHITexture = TextureResource->GetRenderTargetTexture();
	if (!RHITexture)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "AccumulationDOFSVE: Failed to get RHI texture from resource");
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	if (Mode == EAccumulationDOFSVEMode::Capture)
	{
		// Copy scene color to our render target, then pass through unchanged

		FIntVector TextureSize = RHITexture->GetSizeXYZ();

		// Register our external texture as RDG destination
		FRDGTextureRef DestRDG = GraphBuilder.RegisterExternalTexture(
			CreateRenderTarget(RHITexture, TEXT("AccumulationDOFCaptureTarget"))
		);

		// Create a render target binding for the destination
		FScreenPassRenderTarget DestTarget(
			DestRDG,
			FIntRect(0, 0, TextureSize.X, TextureSize.Y),
			ERenderTargetLoadAction::ENoAction
		);

		// Use AddDrawTexturePass to copy scene color to our target

		AddDrawTexturePass(
			GraphBuilder, 
			FScreenPassViewInfo(View),
			FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor),
			DestTarget
		);

		// Pass through unchanged scene color
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
	else
	{
		// Inject

		// Replace scene color with our accumulated texture

		// Register the accumulated texture with RDG
		FRDGTextureRef AccumulatedRDG = GraphBuilder.RegisterExternalTexture(
			CreateRenderTarget(RHITexture, TEXT("AccumulationDOFAccumulated"))
		);

		// Determine output render target
		FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

		// If override output is not provided, we need to create a new target that matches the scene color
		if (!Output.IsValid())
		{
			Output = FScreenPassRenderTarget::CreateFromInput(
				GraphBuilder,
				SceneColor,
				View.GetOverwriteLoadAction(),
				TEXT("AccumulationDOFOutput")
			);
		}

		// Get viewport rect
		const FIntRect ViewRect = SceneColor.ViewRect;

		// Use our injection shader to copy accumulated texture to output.
		// Pass progress bar parameters for preview mode.
		// bApplyAspectFit=false: SceneColor.ViewRect is already at filmback aspect upstream.
		//
		// LensDistortionLUT: only apply in our shader when the engine has routed the LUT to
		// the TSR pass (i.e. it has disabled PrimaryUpscale's LUT path). When PassLocation
		// is PrimaryUpscale, the engine's spatial upscale will apply the LUT itself and
		// applying here would double-distort.
		const FLensDistortionLUT& LensDistortionLUT = LensDistortion::GetLUTUnsafe(View);
		const FLensDistortionLUT* const LUTToApplyInShader =
			(LensDistortion::GetPassLocationUnsafe(View) == LensDistortion::EPassLocation::TSR)
				? &LensDistortionLUT
				: nullptr;

		AccumulationDOF::InjectToSceneColor(
			GraphBuilder,
			AccumulatedRDG,
			Output.Texture,
			ViewRect,
			ProgressBarFraction,
			bDrawProgressBar,
			/*OverscanFraction=*/0.0f,
			/*bDrawPreviewLabel=*/false,
			/*bIsFrozen=*/false,
			/*bApplyAspectFit=*/false,
			LUTToApplyInShader
		);

		return MoveTemp(Output);
	}
}
