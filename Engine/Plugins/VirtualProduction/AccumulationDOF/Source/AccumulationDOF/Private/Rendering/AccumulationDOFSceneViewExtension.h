// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Scene.h"
#include "SceneViewExtension.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UTextureRenderTarget2D;
class UCineCameraComponent;
class FRDGBuilder;
struct FPostProcessMaterialInputs;
struct FScreenPassTexture;

/** Operating mode for the AccumulationDOF Scene View Extension */
enum class EAccumulationDOFSVEMode : uint8
{
	/** Extract scene color at MotionBlur pass to external render target */
	Capture,

	/** Replace scene color at MotionBlur pass with accumulated texture */
	Inject,

	/** Replace scene color before motion blur via temporal upscaler hook */
	InjectViaTemporalUpscaler
};

/**
 * Settings for FAccumulationDOFSceneViewExtension to apply camera post-process settings.
 */
struct FAccumulationDOFSVESettings
{
	/** CineCameraComponent to get post-process settings from during SetupView() */
	TWeakObjectPtr<UCineCameraComponent> CineCameraComponent;

	/**
	 * Create settings from a CineCameraComponent.
	 */
	static FAccumulationDOFSVESettings FromCineCameraComponent(UCineCameraComponent* CineCamera);
};

/**
 * Scene View Extension for AccumulationDOF that hooks into post-processing.
 *
 * The extension operates in three modes (see EAccumulationDOFSVEMode):
 * - Capture: extracts scene color at the MotionBlur pass to an external render target
 * - Inject: replaces scene color at the MotionBlur pass with accumulated texture
 * - InjectViaTemporalUpscaler: replaces scene color before motion blur via temporal upscaler hook
 *
 * This SVE is designed to be created fresh for each capture and added to the SVE array.
 */
class FAccumulationDOFSceneViewExtension : public ISceneViewExtension,
                                        public TSharedFromThis<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe>
{

public:

	/**
	 * @param InRenderTarget - The render target (dst in Capture mode, src in Inject modes)
	 * @param InMode         - The operating mode
	 * @param InSettings     - Optional camera/post-process settings (empty = no PP override)
	 */
	explicit FAccumulationDOFSceneViewExtension(
		UTextureRenderTarget2D* InRenderTarget,
		EAccumulationDOFSVEMode InMode,
		const FAccumulationDOFSVESettings& InSettings = FAccumulationDOFSVESettings()
	);

	virtual ~FAccumulationDOFSceneViewExtension() = default;

	//~ Begin ISceneViewExtension Interface

	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override {}
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;

	virtual void SubscribeToPostProcessingPass(
		EPostProcessingPass Pass,
		const FSceneView& View,
		FAfterPassCallbackDelegateArray& InOutPassCallbacks,
		bool bIsPassEnabled) override;

	virtual int32 GetPriority() const override
	{
		return -10;
	}

	//~ End ISceneViewExtension Interface

	/** Get the current operating mode */
	EAccumulationDOFSVEMode GetMode() const
	{
		return Mode;
	}

	/**
	 * Set DOFSplats parameters for the current sample.
	 *
	 * @param InFStop              - FStop for DOFSplats (0 = disabled)
	 * @param InFocusDistanceCm    - Focus distance in cm
	 * @param InSensorWidthMm      - Sensor width in mm
	 * @param InSqueezeFactor      - Anamorphic squeeze factor
	 * @param bInForceNeutralBokeh - If true, forces petzval=0, circular aperture, and no cat's eye
	 */
	void SetDOFSplatsSettings(float InFStop, float InFocusDistanceCm, float InSensorWidthMm, float InSqueezeFactor, bool bInForceNeutralBokeh = false);

	/**
	 * Set whether to allow SceneFringe (lateral chromatic aberration) in post-process.
	 * When true, engine's SceneFringe will apply during post-processing.
	 * When false, SceneFringeIntensity is forced to 0.
	 */
	void SetAllowSceneFringe(bool bAllow)
	{
		bAllowSceneFringe = bAllow;
	}

	/**
	 * Set progress bar parameters for preview mode.
	 * When enabled, draws a progress bar overlay at the bottom of the frame during injection.
	 * 
	 * @param InProgress - Progress fraction (0.0 to 1.0)
	 */
	void SetProgressBar(float InProgress)
	{
		ProgressBarFraction = InProgress;
		bDrawProgressBar = true;
	}

	/** Get progress bar fraction */
	float GetProgressBarFraction() const
	{
		return ProgressBarFraction;
	}

	/** Check if progress bar should be drawn */
	bool ShouldDrawProgressBar() const
	{
		return bDrawProgressBar;
	}

	/** Get the final blended SceneFringeIntensity value captured during SetupView() */
	float GetCapturedSceneFringeIntensity() const
	{
		return CapturedSceneFringeIntensity;
	}

private:
	/**
	 * Callback for the MotionBlur post-processing pass (@todo possibly move to before)
	 * 
	 * In Capture mode: copies scene color to target RT
	 * In Inject mode: replaces scene color with accumulated texture
	 */
	FScreenPassTexture ProcessAtMotionBlurPass_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FPostProcessMaterialInputs& Inputs);

	/** Operating mode */
	EAccumulationDOFSVEMode Mode;

	/** The render target - destination in Capture mode, source in Inject mode */
	TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/** Settings for camera post-process override (populated at construction) */
	FAccumulationDOFSVESettings Settings;

	//
	// DOFSplats State (per-sample)
	//

	/** DOFSplats f-stop for current sample (0 = disabled) */
	float DOFSplatsFStop = 0.0f;

	/** Focus distance for DOFSplats in cm */
	float DOFSplatsFocusDistanceCm = 0.0f;

	/** Sensor width for DOFSplats in mm */
	float DOFSplatsSensorWidthMm = 0.0f;

	/** Squeeze factor for DOFSplats */
	float DOFSplatsSqueezeFactor = 1.0f;

	/** If true, force neutral bokeh: petzval=0, circular aperture, no cat's eye */
	bool bDOFSplatsForceNeutralBokeh = false;

	/** Whether to allow SceneFringe (lateral chromatic aberration) in post-process or not.*/
	bool bAllowSceneFringe = false;

	//
	// Progress Bar State (for preview mode)
	//

	/** Progress bar fraction (0.0 to 1.0) */
	float ProgressBarFraction = 0.0f;

	/** Whether to draw progress bar overlay during injection */
	bool bDrawProgressBar = false;

	/** Captured SceneFringeIntensity from the final blended PP settings in SetupView() */
	float CapturedSceneFringeIntensity = 0.0f;
};
