// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFTemporalUpscaler.h"
#include "AccumulationDOFShaders.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"

using namespace AccumulationDOF;

FAccumulationDOFTemporalUpscaler::FAccumulationDOFTemporalUpscaler(TRefCountPtr<FRHITexture> InAccumulatedRHITexture, float InProgressBarFraction)
	: AccumulatedRHITexture(InAccumulatedRHITexture)
	, ProgressBarFraction(InProgressBarFraction)
{
}

UE::Renderer::Private::ITemporalUpscaler::FOutputs FAccumulationDOFTemporalUpscaler::AddPasses(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FInputs& Inputs) const
{
	const FIntRect OutputViewRect = Inputs.OutputViewRect;

	// Guard against invalid inputs
	if (!AccumulatedRHITexture ||
		OutputViewRect.Width() <= 0 || OutputViewRect.Height() <= 0 ||
		!Inputs.SceneColor.IsValid() || Inputs.SceneColor.Texture == nullptr)
	{
		FOutputs Outputs;
		Outputs.FullRes = Inputs.SceneColor;
		Outputs.NewHistory = MakeRefCount<FHistory>();
		return Outputs;
	}

	FRDGTextureRef AccumulatedRDG = GraphBuilder.RegisterExternalTexture(
		CreateRenderTarget(AccumulatedRHITexture.GetReference(), TEXT("AccumulationDOFAccumulated"))
	);

	const FIntPoint OutputExtent = Inputs.SceneColor.Texture->Desc.Extent;
	const EPixelFormat OutputFormat = Inputs.SceneColor.Texture->Desc.Format;

	FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
		OutputExtent,
		OutputFormat,
		FClearValueBinding::Black,
		TexCreate_RenderTargetable | TexCreate_ShaderResource
	);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("AccumulationDOFUpscalerOutput"));

	const bool bDrawProgressBar = ProgressBarFraction >= 0.0f;

	// LensDistortionLUT is not applied here: the parent SVE forces TemporalUpscale on
	// the view so the engine routes the LUT to PrimaryUpscale, which applies it after
	// our upscaler runs. Applying here would double-distort.
	//
	// bApplyAspectFit=false: OutputViewRect is already at filmback aspect upstream.
	InjectToSceneColor(
		GraphBuilder,
		AccumulatedRDG,
		OutputTexture,
		OutputViewRect,
		ProgressBarFraction,
		bDrawProgressBar,
		/*OverscanFraction=*/0.0f,
		/*bDrawPreviewLabel=*/false,
		/*bIsFrozen=*/false,
		/*bApplyAspectFit=*/false
	);

	FOutputs Outputs;

	Outputs.FullRes = FScreenPassTexture(OutputTexture, OutputViewRect);
	Outputs.NewHistory = MakeRefCount<FHistory>();

	return Outputs;
}

UE::Renderer::Private::ITemporalUpscaler* FAccumulationDOFTemporalUpscaler::Fork_GameThread(
	const FSceneViewFamily& ViewFamily) const
{
	return new FAccumulationDOFTemporalUpscaler(AccumulatedRHITexture, ProgressBarFraction);
}
