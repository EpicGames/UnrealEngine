// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "ScreenPass.h"
#include "Templates/RefCounting.h"
#include "TemporalUpscaler.h"

/**
 * Custom temporal upscaler that injects the accumulated AccumulationDOF result
 * before motion blur runs. This allows motion blur to be applied to the
 * accumulated depth of field result.
 *
 * We do it like this because there currently isn´t a "BeforeMotionBlur" SVE callback available.
 * The samples themselves can still use an upscaler, but since we're injecting an accumulation and discard
 * the upscaler render, it doesn't affect the output.
 */
class FAccumulationDOFTemporalUpscaler : public UE::Renderer::Private::ITemporalUpscaler
{
public:

	/**
	 * History object required by the ITemporalUpscaler interface.
	 * Required for interface (GetDebugName must match upscaler), but we don't store
	 * temporal state since we inject an already accumulated buffer.
	 */
	class FHistory : public IHistory, public TRefCountingMixin<FHistory>
	{
	public:
		virtual const TCHAR* GetDebugName() const override
		{
			return GetUpscalerDebugName();
		}

		virtual uint64 GetGPUSizeBytes() const override
		{
			return 0;
		}

		virtual void AddRef() const override
		{
			TRefCountingMixin<FHistory>::AddRef();
		}

		virtual FReturnedRefCountValue Release() const override
		{
			return TRefCountingMixin<FHistory>::Release();
		}

		virtual FReturnedRefCountValue GetRefCount() const override
		{
			return TRefCountingMixin<FHistory>::GetRefCount();
		}
	};

	/** Debug name */
	static const TCHAR* GetUpscalerDebugName()
	{
		return TEXT("AccumulationDOF");
	}

	FAccumulationDOFTemporalUpscaler(TRefCountPtr<FRHITexture> InAccumulatedRHITexture, float InProgressBarFraction = -1.0f);

	virtual ~FAccumulationDOFTemporalUpscaler() = default;

	// ITemporalUpscaler interface
	virtual const TCHAR* GetDebugName() const override
	{
		return GetUpscalerDebugName();
	}

	/**
	 * Add the injection pass to the render graph.
	 */
	virtual FOutputs AddPasses(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FInputs& Inputs) const override;

	virtual float GetMinUpsampleResolutionFraction() const override
	{
		return 1.0f;
	}

	virtual float GetMaxUpsampleResolutionFraction() const override
	{
		return 1.0f;
	}

	virtual ITemporalUpscaler* Fork_GameThread(const class FSceneViewFamily& ViewFamily) const override;

private:

	/** Accumulated RT, for render thread access */
	TRefCountPtr<FRHITexture> AccumulatedRHITexture;

	/** Progress bar fraction (0-1), or negative to disable */
	float ProgressBarFraction;
};
