// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApertureSampler.h"

#include "ApertureSamplingRenderer.h"
#include "AccumulationDOFLog.h"
#include "AccumulationDOFMath.h"
#include "AccumulationDOFShaders.h"
#include "AccumulationDOFUtils.h"

#include "CineCameraComponent.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderingThread.h"
#include "RHIStaticStates.h"
#include "TextureResource.h"

static TAutoConsoleVariable<int32> CVarAccumulationDOFPostMedianFilterEnable(
	TEXT("r.AccumulationDOF.PostMedianFilter.Enable"),
	0,
	TEXT("Enable the 3x3 median filter in Accumulation DOF.\n")
	TEXT("  0: Disabled\n")
	TEXT("  1: Enabled"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarAccumulationDOFPreMedianFilterEnable(
	TEXT("r.AccumulationDOF.PreMedianFilter.Enable"),
	0,
	TEXT("Apply 3x3 median filter to each sample before accumulation.\n")
	TEXT("  0: Disabled\n")
	TEXT("  1: Enabled"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarAccumulationDOFJitterAAAllow(
	TEXT("r.AccumulationDOF.JitterAA.Allow"),
	1,
	TEXT("Allow jitter AA in Accumulation DOF.\n")
	TEXT("  0: Disabled\n")
	TEXT("  1: Allowed"),
	ECVF_Default
);

// Jitter sequence types
enum class EJitterSequence : int32
{
	Halton = 0,
	Sobol = 1,
};

static TAutoConsoleVariable<int32> CVarAccumulationDOFJitterAASequence(
	TEXT("r.AccumulationDOF.JitterAA.Sequence"),
	static_cast<int32>(EJitterSequence::Halton),
	TEXT("Jitter sequence for AA.\n")
	TEXT("  0: Halton\n")
	TEXT("  1: Sobol"),
	ECVF_Default
);

// Reconstruction filter types
enum class EAAFilter : int32
{
	Box = 0,
	Mitchell = 1,
};

static TAutoConsoleVariable<int32> CVarAccumulationDOFJitterAAFilter(
	TEXT("r.AccumulationDOF.JitterAA.Filter"),
	static_cast<int32>(EAAFilter::Box),
	TEXT("Reconstruction filter for AA.\n")
	TEXT("  0: Box\n")
	TEXT("  1: Mitchell-Netravali"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarAccumulationDOFJitterAABilinearSplat(
	TEXT("r.AccumulationDOF.JitterAA.BilinearSplat"),
	1,
	TEXT("Enable bilinear subpixel splatting (via gather) for AA reconstruction \n")
	TEXT("  0: Disabled (point sampling)\n")
	TEXT("  1: Enabled (default)"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarAccumulationDOFApertureJitterAllowed(
	TEXT("r.AccumulationDOF.ApertureJitter.Allowed"),
	1,
	TEXT("Allow aperture position jittering.\n")
	TEXT("  0: All samples render from aperture center\n")
	TEXT("  1: Normal aperture sampling"),
	ECVF_Default
);

using namespace AccumulationDOF;

//
// Spectral LUT for axial chromatic aberration
// Input: Wavelengths 380-700nm
// Output: Linear sRGB, normalized such that R+G+B = 1
//
static constexpr float SpectralLUT[32][3] = {
	{ 0.175237f, 0.000000f, 0.824763f }, // 380.0 nm
	{ 0.167134f, 0.000000f, 0.832866f }, // 390.3 nm
	{ 0.149273f, 0.000000f, 0.850727f }, // 400.6 nm
	{ 0.113597f, 0.000000f, 0.886403f }, // 411.0 nm
	{ 0.062576f, 0.000000f, 0.937424f }, // 421.3 nm
	{ 0.017058f, 0.046903f, 0.936039f }, // 431.6 nm
	{ 0.000000f, 0.149028f, 0.850972f }, // 441.9 nm
	{ 0.000000f, 0.258459f, 0.741541f }, // 452.3 nm
	{ 0.000000f, 0.360560f, 0.639440f }, // 462.6 nm
	{ 0.000000f, 0.458963f, 0.541037f }, // 472.9 nm
	{ 0.000000f, 0.558498f, 0.441502f }, // 483.2 nm
	{ 0.000000f, 0.660484f, 0.339516f }, // 493.5 nm
	{ 0.000000f, 0.763419f, 0.236581f }, // 503.9 nm
	{ 0.000000f, 0.860713f, 0.139287f }, // 514.2 nm
	{ 0.000000f, 0.940980f, 0.059020f }, // 524.5 nm
	{ 0.075869f, 0.924131f, 0.000000f }, // 534.8 nm
	{ 0.225498f, 0.774502f, 0.000000f }, // 545.2 nm
	{ 0.359627f, 0.640373f, 0.000000f }, // 555.5 nm
	{ 0.479421f, 0.520579f, 0.000000f }, // 565.8 nm
	{ 0.585598f, 0.414402f, 0.000000f }, // 576.1 nm
	{ 0.680139f, 0.319861f, 0.000000f }, // 586.5 nm
	{ 0.763553f, 0.236447f, 0.000000f }, // 596.8 nm
	{ 0.835816f, 0.164184f, 0.000000f }, // 607.1 nm
	{ 0.896155f, 0.103845f, 0.000000f }, // 617.4 nm
	{ 0.943269f, 0.056731f, 0.000000f }, // 627.7 nm
	{ 0.975068f, 0.024932f, 0.000000f }, // 638.1 nm
	{ 0.992392f, 0.007608f, 0.000000f }, // 648.4 nm
	{ 0.998694f, 0.001306f, 0.000000f }, // 658.7 nm
	{ 0.999918f, 0.000082f, 0.000000f }, // 669.0 nm
	{ 1.000000f, 0.000000f, 0.000000f }, // 679.4 nm
	{ 1.000000f, 0.000000f, 0.000000f }, // 689.7 nm
	{ 1.000000f, 0.000000f, 0.000000f }, // 700.0 nm
};

UApertureSampler::UApertureSampler()
{
}

bool UApertureSampler::Initialize(const FApertureSamplerConfig& InConfig, const FApertureSamplerCameraState& InCameraState)
{
	Config = InConfig;
	CameraState = InCameraState;
	BokehTextureRef = Config.BokehTexture;

	// Validate configuration

	if (!Config.World.IsValid())
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::Initialize: World is null");
		return false;
	}

	if (Config.NumSamples < 1)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::Initialize: NumSamples must be >= 1");
		return false;
	}

	if (Config.Resolution.X < 64 || Config.Resolution.Y < 64)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::Initialize: Invalid resolution");
		return false;
	}

	SetupRenderTargets();

	if (!SampleRT || !AccumA || !AccumB || !PrefilterRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::Initialize: Failed to create render targets");
		return false;
	}

	InitializeRenderer();

	if (!Renderer || !Renderer->IsInitialized())
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::Initialize: Failed to initialize renderer");
		return false;
	}

	GenerateApertureOffsets();

	// Validate that samples were generated
	if (Progress.ActualNumSamples == 0 || ApertureOffsetsCm.IsEmpty())
	{
		UE_LOGF(LogAccumulationDOF, Warning, "UApertureSampler::Initialize: Failed to generate aperture samples");
		return false;
	}

	PrecomputeSpectralWeights();

	// Initialize progress
	Progress.CurrentIteration = 0;
	Progress.TotalIterations = GetTotalIterations();
	Progress.bComplete = false;

	// Cache initial state for change detection
	CacheCurrentState();
	bLastChangeDetectionValid = true;

	// Force full-resolution DOF gather when using DOF splats
	if (Config.DOFSplatSize > UE_KINDA_SMALL_NUMBER)
	{
		static IConsoleVariable* DOFGatherResDivisorCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DOF.Gather.ResolutionDivisor"));
		if (DOFGatherResDivisorCVar)
		{
			SavedDOFGatherResDivisor = DOFGatherResDivisorCVar->GetInt();
			if (SavedDOFGatherResDivisor != 1)
			{
				DOFGatherResDivisorCVar->Set(1, ECVF_SetByCode);
				bModifiedDOFGatherResDivisor = true;
			}
		}
	}

	bIsInitialized = true;
	bHasValidAccumulatedData = false;

	return true;
}

void UApertureSampler::Shutdown()
{
	if (Renderer)
	{
		Renderer->Shutdown();
	}

	// Restore DOF gather resolution divisor if we modified it
	if (bModifiedDOFGatherResDivisor)
	{
		static IConsoleVariable* DOFGatherResDivisorCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DOF.Gather.ResolutionDivisor"));
		if (DOFGatherResDivisorCVar)
		{
			DOFGatherResDivisorCVar->Set(SavedDOFGatherResDivisor, ECVF_SetByCode);
		}
		bModifiedDOFGatherResDivisor = false;
	}

	ApertureOffsetsCm.Empty();

	Progress = AccumulationDOF::FApertureSamplerProgress();
	TotalSpectralWeightPerChannel = FVector3f::ZeroVector;
	CameraState.SceneFringeIntensity = 0.0f;

	if (SampleRT || AccumA || AccumB || PrefilterRT || PostProcessRT)
	{
		FlushRenderingCommands();
	}

	SampleRT = nullptr;
	AccumA = nullptr;
	AccumB = nullptr;
	PrefilterRT = nullptr;
	PostProcessRT = nullptr;

	bIsInitialized = false;
	bHasValidAccumulatedData = false;
}

void UApertureSampler::SetupRenderTargets()
{
	const int32 Width = Config.Resolution.X;
	const int32 Height = Config.Resolution.Y;

	// Accumulate in 32 bit linear HDR. 16-bit may run into banding when accumulating many frames.
	ETextureRenderTargetFormat TargetFormat = RTF_RGBA32f;

	UWorld* World = Config.World.Get();
	if (!World)
	{
		return;
	}

	// Create render targets (GC handled via UPROPERTY)
	auto CreateRT = [&](TObjectPtr<UTextureRenderTarget2D>& RT, const TCHAR* Name) -> bool
	{
		if (!RT || RT->SizeX != Width || RT->SizeY != Height)
		{
			RT = NewObject<UTextureRenderTarget2D>(this, Name);
			if (RT)
			{
				RT->RenderTargetFormat = TargetFormat;
				RT->ClearColor = FLinearColor::Transparent;
				RT->bAutoGenerateMips = false;
				RT->InitAutoFormat(Width, Height);
				RT->UpdateResourceImmediate(true);
			}
		}
		return RT != nullptr;
	};

	if (!CreateRT(SampleRT, TEXT("AccumulationDOF_SampleRT")))
	{
		UE_LOGF(LogAccumulationDOF, Error, "Failed to create SampleRT");
		return;
	}

	if (!CreateRT(AccumA, TEXT("AccumulationDOF_AccumA")))
	{
		UE_LOGF(LogAccumulationDOF, Error, "Failed to create AccumA");
		return;
	}

	if (!CreateRT(AccumB, TEXT("AccumulationDOF_AccumB")))
	{
		UE_LOGF(LogAccumulationDOF, Error, "Failed to create AccumB");
		return;
	}

	if (!CreateRT(PrefilterRT, TEXT("AccumulationDOF_PrefilterRT")))
	{
		UE_LOGF(LogAccumulationDOF, Error, "Failed to create PrefilterRT");
		return;
	}
}

void UApertureSampler::InitializeRenderer()
{
	if (!Renderer)
	{
		Renderer = NewObject<UApertureSamplingRenderer>(this);
	}

	// (Re)initialize with the current world and exposure view state
	Renderer->Initialize(Config.World.Get(), Config.ExposureViewState);

	// Configure the renderer
	FApertureSamplingConfig RendererConfig;
	RendererConfig.Resolution = Config.Resolution;
	RendererConfig.CaptureSource = ESceneCaptureSource::SCS_FinalColorHDR;
	RendererConfig.bDisableEngineDOF = true;
	RendererConfig.ScreenPercentageFraction = Config.ScreenPercentage;

	// Configure ShowFlags
	RendererConfig.ShowFlags = FEngineShowFlags(ESFIM_Game);
	RendererConfig.ShowFlags.SetDepthOfField(false);
	RendererConfig.ShowFlags.SetMotionBlur(false);

	// AA configuration based on jitter mode
	if (Config.bUseJitterAA)
	{
		RendererConfig.ShowFlags.SetTemporalAA(false);
		RendererConfig.ShowFlags.SetAntiAliasing(false);
	}
	else
	{
		RendererConfig.ShowFlags.SetTemporalAA(true);
		RendererConfig.ShowFlags.SetAntiAliasing(true);
	}

	RendererConfig.ViewModeIndex = Config.ViewModeIndex;

	Renderer->SetupRenderTargets(RendererConfig);
}

void UApertureSampler::GenerateApertureOffsets()
{
	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::GenerateApertureOffsets: No camera component");
		ApertureOffsetsCm.Empty();
		Progress.ActualNumSamples = 0;
		return;
	}

	const float FocalLengthMm = CineCam->CurrentFocalLength;
	const float FStop = CineCam->CurrentAperture;
	const float ApertureRadiusCm = AccumulationDOFMath::GetApertureRadiusCm(FocalLengthMm, FStop);

	switch (Config.SamplingPattern)
	{
	case EApertureSamplingPattern::Hexaweb:
		{
			const int32 NumRings = AccumulationDOFMath::ComputeRingCountForSamples(Config.NumSamples);
			Progress.ActualNumSamples = AccumulationDOFMath::GetRingSampleCount(NumRings);
			AccumulationDOFMath::GenerateApertureSamplesRing(NumRings, ApertureRadiusCm, ApertureOffsetsCm);
			AccumulationDOFMath::ReorderSamplesForPendulum(ApertureOffsetsCm);
		}
		break;

	case EApertureSamplingPattern::Vogel:
		Progress.ActualNumSamples = Config.NumSamples;
		AccumulationDOFMath::GenerateApertureSamplesVogel(Config.NumSamples, ApertureRadiusCm, ApertureOffsetsCm);
		AccumulationDOFMath::ReorderSamplesForPendulum(ApertureOffsetsCm);
		break;

	case EApertureSamplingPattern::Halton:
	default:
		Progress.ActualNumSamples = Config.NumSamples;
		AccumulationDOFMath::GenerateApertureSamplesHalton(Config.NumSamples, ApertureRadiusCm, ApertureOffsetsCm);
		break;
	}

	Progress.TotalIterations = GetTotalIterations();
}

void UApertureSampler::PrecomputeSpectralWeights()
{
	TotalSpectralWeightPerChannel = FVector3f(0.0f);

	const int32 EffectiveBands = GetEffectiveBands();
	if (EffectiveBands <= 1)
	{
		TotalSpectralWeightPerChannel = FVector3f(1.0f);
		return;
	}

	for (int32 Band = 0; Band < EffectiveBands; ++Band)
	{
		const float NormalizedSpectralPosition = static_cast<float>(Band) / static_cast<float>(EffectiveBands - 1);
		const FVector3f Weight = ComputeSpectralWeight(NormalizedSpectralPosition);
		TotalSpectralWeightPerChannel += Weight;
	}
}

bool UApertureSampler::UpdateCameraState(const FApertureSamplerCameraState& InCameraState)
{
	CameraState = InCameraState;

	// Reset accumulation if the user changed one of the critical camera settings we monitor

	if (HasSignificantChanges())
	{
		ResetAccumulation();
		return true;
	}

	return false;
}

void UApertureSampler::ResetAccumulation()
{
	ClearAccumulationBuffers();
	CacheCurrentState();
	bLastChangeDetectionValid = true;

	// Regenerate aperture offsets in case aperture changed
	GenerateApertureOffsets();

	// Pre-compute spectral weights
	PrecomputeSpectralWeights();
}

void UApertureSampler::ClearAccumulationBuffers()
{
	Progress.CurrentIteration = 0;
	Progress.bComplete = false;
	bHasValidAccumulatedData = false;
	AccumulatedWeightSum = 0.0f;

	// Clear accumulation buffers with transparent black (alpha=0) for weight accumulation

	if (UWorld* World = Config.World.Get())
	{
		if (AccumA)
		{
			UKismetRenderingLibrary::ClearRenderTarget2D(World, AccumA, FLinearColor::Transparent);
		}

		if (AccumB)
		{
			UKismetRenderingLibrary::ClearRenderTarget2D(World, AccumB, FLinearColor::Transparent);
		}
	}
}

bool UApertureSampler::HasSignificantChanges() const
{
	if (!bLastChangeDetectionValid)
	{
		return false;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();

	if (!CineCam)
	{
		return true;
	}

	// Camera transform change (position or rotation)

	FTransform CurrentTransform(CameraState.CameraRotation, CameraState.CameraLocation);

	if (!CurrentTransform.Equals(LastCameraTransform, 0.01f))  // 0.01cm threshold
	{
		return true;
	}

	// Focus distance change
	if (!FMath::IsNearlyEqual(CineCam->CurrentFocusDistance, LastFocusDistance, 0.1f))
	{
		return true;
	}

	// Aperture change (f-stop)
	if (!FMath::IsNearlyEqual(CineCam->CurrentAperture, LastAperture, 0.01f))
	{
		return true;
	}

	// Focal length change
	if (!FMath::IsNearlyEqual(CineCam->CurrentFocalLength, LastFocalLength, 0.1f))
	{
		return true;
	}

	return false;
}

void UApertureSampler::CacheCurrentState()
{
	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (CineCam)
	{
		LastCameraTransform = FTransform(CameraState.CameraRotation, CameraState.CameraLocation);
		LastFocusDistance = CineCam->CurrentFocusDistance;
		LastAperture = CineCam->CurrentAperture;
		LastFocalLength = CineCam->CurrentFocalLength;
		bLastChangeDetectionValid = true;
	}
}

bool UApertureSampler::RenderAllSamples()
{
	if (!bIsInitialized)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::RenderAllSamples: Not initialized");
		return false;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::RenderAllSamples: No CineCameraComponent");
		return false;
	}

	Progress.bWasCancelled = false;

	const int32 TotalIterations = GetTotalIterations();
	FScopedSlowTask SlowTask(TotalIterations, FText::FromString(TEXT("Rendering Accumulation DOF...")));
	SlowTask.MakeDialog(true);

	// Clear accumulation
	ClearAccumulationBuffers();

	const bool bHasAxialCA = Config.AxialChromaticAberrationIntensity > UE_KINDA_SMALL_NUMBER;

	if (bHasAxialCA)
	{
		const int32 EffectiveBands = GetEffectiveBands();

		// Render all bands for each sample
		const FVector3f InvTotal(
			(TotalSpectralWeightPerChannel.X > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.X) : 0.0f,
			(TotalSpectralWeightPerChannel.Y > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.Y) : 0.0f,
			(TotalSpectralWeightPerChannel.Z > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.Z) : 0.0f
		);

		int32 IterationCount = 0;
		for (int32 SampleIdx = 0; SampleIdx < Progress.ActualNumSamples && !Progress.bWasCancelled; ++SampleIdx)
		{
			for (int32 Band = 0; Band < EffectiveBands; ++Band)
			{
				if (SlowTask.ShouldCancel())
				{
					Progress.bWasCancelled = true;
					break;
				}

				const float NormalizedSpectralPosition = (EffectiveBands == 1) ? 0.5f : static_cast<float>(Band) / static_cast<float>(EffectiveBands - 1);
				const float FocusDistBand = ComputeSpectralFocusDistance(CineCam->CurrentFocusDistance, NormalizedSpectralPosition);

				const FVector3f RawWeight = ComputeSpectralWeight(NormalizedSpectralPosition);
				const FVector3f SpectralWeight = RawWeight * InvTotal;

				const FVector2f SampleApertureOffsetCm = GetApertureOffset(SampleIdx);
				RenderSample(SampleIdx, SampleApertureOffsetCm, FocusDistBand);

				// Save intermediate sample if requested
				if (Config.bSaveIntermediateSamples)
				{
					SaveSampleImage(IterationCount, Config.OutputDirectory, Config.OutputFilenamePrefix);
				}

				AccumulateSampleInternal(SampleIdx, SampleApertureOffsetCm, SpectralWeight);

				IterationCount++;
				Progress.CurrentIteration = IterationCount;
				NotifyProgress();
				SlowTask.EnterProgressFrame(1);

				// Flush GPU commands periodically for UI responsiveness
				if ((IterationCount % AccumulationDOFUtils::UIFlushBatchSize) == 0)
				{
					FlushRenderingCommands();
				}
				if ((IterationCount % AccumulationDOFUtils::RenderingMemoryFlushBatchSize) == 0)
				{
					AccumulationDOFUtils::ReleaseRenderingMemory();
				}
			}
		}

		AccumulationDOFUtils::ReleaseRenderingMemory();

		if (Progress.bWasCancelled)
		{
			return false;
		}

		// Normalize by number of samples (spectral weights already sum to 1 per channel)
		NormalizeAccumulation(static_cast<float>(Progress.ActualNumSamples));
	}
	else
	{
		// No axial chromatic aberration
		for (int32 SampleIdx = 0; SampleIdx < Progress.ActualNumSamples; ++SampleIdx)
		{
			if (SlowTask.ShouldCancel())
			{
				Progress.bWasCancelled = true;
				break;
			}

			const FVector2f SampleApertureOffsetCm = GetApertureOffset(SampleIdx);
			RenderSample(SampleIdx, SampleApertureOffsetCm);

			// Save intermediate sample if requested
			if (Config.bSaveIntermediateSamples)
			{
				SaveSampleImage(SampleIdx, Config.OutputDirectory, Config.OutputFilenamePrefix);
			}

			AccumulateSampleInternal(SampleIdx, SampleApertureOffsetCm, FVector3f(1.0f));

			Progress.CurrentIteration = SampleIdx + 1;
			NotifyProgress();
			SlowTask.EnterProgressFrame(1);

			// Flush GPU commands periodically for UI responsiveness
			if (((SampleIdx + 1) % AccumulationDOFUtils::UIFlushBatchSize) == 0)
			{
				FlushRenderingCommands();
			}
			if (((SampleIdx + 1) % AccumulationDOFUtils::RenderingMemoryFlushBatchSize) == 0)
			{
				AccumulationDOFUtils::ReleaseRenderingMemory();
			}
		}

		AccumulationDOFUtils::ReleaseRenderingMemory();

		if (Progress.bWasCancelled)
		{
			return false;
		}

		NormalizeAccumulation(static_cast<float>(Progress.ActualNumSamples));
	}

	Progress.bComplete = true;
	NotifyProgress();

	// Save final accumulated image if requested
	if (Config.bSaveFinalAccumulation)
	{
		SaveFinalImage(Config.OutputDirectory, Config.OutputFilenamePrefix);
	}

	if (OnCompleteDelegate.IsBound())
	{
		OnCompleteDelegate.Execute(AccumA);
	}

	return true;
}

bool UApertureSampler::RenderAmortizedSamples()
{
	if (!bIsInitialized)
	{
		return false;
	}

	if (Progress.bComplete)
	{
		return false;
	}

	if (Config.SamplesPerFrame <= 0)
	{
		return false;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		return false;
	}

	const bool bHasAxialCA = Config.AxialChromaticAberrationIntensity > UE_KINDA_SMALL_NUMBER;
	const int32 EffectiveBands = GetEffectiveBands();
	const int32 TotalIterations = GetTotalIterations();

	const FVector3f InvTotal(
		(TotalSpectralWeightPerChannel.X > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.X) : 0.0f,
		(TotalSpectralWeightPerChannel.Y > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.Y) : 0.0f,
		(TotalSpectralWeightPerChannel.Z > UE_KINDA_SMALL_NUMBER) ? (1.0f / TotalSpectralWeightPerChannel.Z) : 0.0f
	);

	int32 SamplesRendered = 0;

	while (SamplesRendered < Config.SamplesPerFrame && Progress.CurrentIteration < TotalIterations)
	{
		int32 SampleIndex = Progress.CurrentIteration;
		int32 Band = 0;
		float NormalizedSpectralPosition = 0.5f;

		if (bHasAxialCA)
		{
			SampleIndex = Progress.CurrentIteration / EffectiveBands;
			Band = Progress.CurrentIteration % EffectiveBands;
			NormalizedSpectralPosition = (EffectiveBands == 1) ? 0.5f : static_cast<float>(Band) / static_cast<float>(EffectiveBands - 1);
		}

		if (SampleIndex >= ApertureOffsetsCm.Num())
		{
			break;
		}

		const FVector2f ApertureOffsetCm = GetApertureOffset(SampleIndex);

		FVector3f SpectralWeight(1.0f);
		float FocusDistanceOverride = -1.0f;

		if (bHasAxialCA)
		{
			FocusDistanceOverride = ComputeSpectralFocusDistance(CineCam->CurrentFocusDistance, NormalizedSpectralPosition);
			const FVector3f RawWeight = ComputeSpectralWeight(NormalizedSpectralPosition);
			SpectralWeight = RawWeight * InvTotal;
		}

		RenderSample(SampleIndex, ApertureOffsetCm, FocusDistanceOverride);

		AccumulateSampleInternal(SampleIndex, ApertureOffsetCm, SpectralWeight);

		SamplesRendered++;
		Progress.CurrentIteration++;
	}

	NotifyProgress();

	// Check if complete
	if (Progress.CurrentIteration >= TotalIterations)
	{
		NormalizeAccumulation(static_cast<float>(Progress.ActualNumSamples));
		Progress.bComplete = true;

		if (OnCompleteDelegate.IsBound())
		{
			OnCompleteDelegate.Execute(AccumA);
		}
	}

	return true;
}

void UApertureSampler::RenderSample(int32 SampleIndex, const FVector2f& ApertureOffsetCm, float FocusDistanceOverride)
{
	if (!Renderer || !Renderer->IsInitialized())
	{
		UE_LOGF(LogAccumulationDOF, Error, "UApertureSampler::RenderSample: Renderer not initialized");
		return;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam || !SampleRT)
	{
		return;
	}

	// Compute sample parameters
	FApertureSampleParams Params = ComputeSampleParams(SampleIndex, FocusDistanceOverride);

	if (Params.SampleIndex == INDEX_NONE)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "RenderSample: Invalid sample params for SampleIndex=%d (projection computation failed)", SampleIndex);
		return;
	}

	// Render the sample
	float CapturedSceneFringe = 0.0f;
	if (Renderer->RenderSample(Params, SampleRT, CineCam, &CapturedSceneFringe))
	{
		CameraState.SceneFringeIntensity = CapturedSceneFringe;
	}
}

FApertureSampleParams UApertureSampler::ComputeSampleParams(int32 SampleIndex, float FocusDistanceOverride) const
{
	FApertureSampleParams Params;

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		return Params;
	}

	// Single GetCameraView per sample.
	FMinimalViewInfo CameraView;
	CineCam->GetCameraView(0.0f, CameraView);

	const FVector2f ApertureOffsetCm = GetApertureOffset(SampleIndex);

	// Compute projection matrix
	FMatrix ProjectionMatrix;
	if (!ComputeProjectionMatrix(SampleIndex, FocusDistanceOverride, CameraView, ProjectionMatrix))
	{
		return Params;
	}

	// Read camera parameters
	const float FocalLengthMm = CineCam->CurrentFocalLength;
	const float FStop = CineCam->CurrentAperture;
	const float FocalLengthCm = AccumulationDOFMath::MmToCm(FocalLengthMm);
	const float ApertureRadiusCm = AccumulationDOFMath::GetApertureRadiusCm(FocalLengthMm, FStop);

	// Use override focus distance if provided
	float FocusDistanceCm = (FocusDistanceOverride >= 0.0f) ? FocusDistanceOverride : CineCam->CurrentFocusDistance;

	// Apply spherical aberration if enabled
	if (Config.SphericalAberration > UE_KINDA_SMALL_NUMBER)
	{
		const float DeltaZCm = AccumulationDOFMath::ComputeSphericalAberrationShift(
			ApertureOffsetCm, ApertureRadiusCm, FocalLengthCm, Config.SphericalAberration
		);
		FocusDistanceCm += DeltaZCm;
		FocusDistanceCm = FMath::Max(FocusDistanceCm, 1.0f);
	}

	const float SensorWidthMm = CineCam->Filmback.SensorWidth;
	const float SensorHeightMm = CineCam->Filmback.SensorHeight;
	const float SqueezeFactor = CineCam->LensSettings.SqueezeFactor;

	// Compute DOF splat f-stop if enabled (size > 0)
	float DOFSplatsFStop = 0.0f;
	if (Config.DOFSplatSize > UE_KINDA_SMALL_NUMBER)
	{
		DOFSplatsFStop = AccumulationDOFMath::ComputeDOFSplatsFStop(ApertureRadiusCm, FocalLengthMm, Config.DOFSplatSize);
	}

	// Build parameters.
	Params.CameraLocation     = CameraState.CameraLocation;
	Params.CameraRotation     = CameraState.CameraRotation;
	Params.ApertureOffsetCm   = ApertureOffsetCm;
	Params.ProjectionMatrix   = ProjectionMatrix;
	Params.FOVDegrees         = CameraView.FOV;              // Used by groom
	Params.FocalLengthMm      = FocalLengthMm;
	Params.FStop              = FStop;
	Params.FocusDistanceCm    = FocusDistanceCm;
	Params.SensorWidthMm      = SensorWidthMm;
	Params.SensorHeightMm     = SensorHeightMm;
	Params.SqueezeFactor      = SqueezeFactor;
	Params.DOFSplatsFStop     = DOFSplatsFStop;
	Params.bForceNeutralBokeh = true;

	// Determine bWorldIsPaused based on TemporalHistoryMode
	const bool bIsFirstSample = (SampleIndex == 0);
	const bool bIsLastSample  = (SampleIndex == Progress.ActualNumSamples - 1);

	switch (Config.TemporalHistoryMode)
	{
	case ETemporalHistoryMode::AllSamplesUpdate:
		Params.bWorldIsPaused = false;
		break;

	case ETemporalHistoryMode::FirstSampleOnly:
		Params.bWorldIsPaused = !bIsFirstSample;
		break;

	case ETemporalHistoryMode::NoSamplesUpdate:
		Params.bWorldIsPaused = true;
		break;

	case ETemporalHistoryMode::LastSampleOnly:
		Params.bWorldIsPaused = !bIsLastSample;
		break;

	default:
		checkNoEntry();
		break;
	}

	UE_LOGF(LogAccumulationDOF, Verbose, "ComputeSampleParams[%d]: TemporalHistoryMode=%d, bWorldIsPaused=%d, bIsFirstSample=%d, bIsLastSample=%d",
		SampleIndex, (int32)Config.TemporalHistoryMode, Params.bWorldIsPaused, bIsFirstSample, bIsLastSample);

	Params.AntiAliasing      = Config.bUseJitterAA ? EAntiAliasingMethod::AAM_None : EAntiAliasingMethod::AAM_TemporalAA; // TAA can help DOF splats
	Params.bEnableMotionBlur = false;
	Params.bUseRayTracing    = true;
	Params.SampleIndex       = SampleIndex;
	Params.DOFSensorScale    = 1.0f;
	Params.bAllowSceneFringe = false;

	return Params;
}

bool UApertureSampler::ComputeProjectionMatrix(
	int32 SampleIndex,
	float FocusDistanceOverride,
	const FMinimalViewInfo& InCameraView,
	FMatrix& OutMatrix
) const
{
	OutMatrix = FMatrix::Identity;

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();

	if (!CineCam)
	{
		return false;
	}

	const FVector2f ApertureOffsetCm = GetApertureOffset(SampleIndex);

	// Camera vectors
	const FQuat CameraQuat = CameraState.CameraRotation.Quaternion();
	const FVector RightVec = CameraQuat.GetRightVector();
	const FVector UpVec = CameraQuat.GetUpVector();
	const FVector ForwardVec = CameraQuat.GetForwardVector();

	// Camera parameters
	const float FocalLengthMm = CineCam->CurrentFocalLength;
	const float FocalLengthCm = AccumulationDOFMath::MmToCm(FocalLengthMm);
	const float ApertureRadiusCm = AccumulationDOFMath::GetApertureRadiusCm(FocalLengthMm, CineCam->CurrentAperture);

	float FocusDistanceCm = (FocusDistanceOverride >= 0.0f) ? FocusDistanceOverride : CineCam->CurrentFocusDistance;

	// Apply spherical aberration
	if (Config.SphericalAberration > UE_KINDA_SMALL_NUMBER)
	{
		const float DeltaZCm = AccumulationDOFMath::ComputeSphericalAberrationShift(
			ApertureOffsetCm, ApertureRadiusCm, FocalLengthCm, Config.SphericalAberration
		);
		FocusDistanceCm += DeltaZCm;
		FocusDistanceCm = FMath::Max(FocusDistanceCm, 1.0f);
	}

	const float NearCm = InCameraView.GetFinalPerspectiveNearClipPlane();
	const float FarCm = NearCm;  // Infinite far plane

	// Derive an effective sensor that already encodes everything the engine view applied.
	// InCameraView.FOV already includes squeeze + crop aspect, then ApplyOverscan + ApplyAsymmetricOverscan
	// in the view setup fold both overscan kinds into FOV / AspectRatio. 

	const float HalfFOVRad        = FMath::DegreesToRadians(0.5f * InCameraView.FOV);
	const float SensorHalfWidthCm = FocalLengthCm * FMath::Tan(HalfFOVRad);
	const float ViewAspectRatio   = FMath::Max(InCameraView.AspectRatio, UE_KINDA_SMALL_NUMBER);
	const float EffectiveSensorWidthCm  = 2.0f * SensorHalfWidthCm;
	const float EffectiveSensorHeightCm = EffectiveSensorWidthCm / ViewAspectRatio;

	// Compute sensor corners
	FVector LocalSensorCorners[4];
	AccumulationDOFMath::ComputeSensorCorners(EffectiveSensorWidthCm, EffectiveSensorHeightCm, FocalLengthCm, LocalSensorCorners);

	// Apply the engine's combined off-center projection offset.
	if (!InCameraView.OffCenterProjectionOffset.IsNearlyZero())
	{
		const float SensorOffsetXCm = static_cast<float>(InCameraView.OffCenterProjectionOffset.X) * EffectiveSensorWidthCm  * 0.5f;
		const float SensorOffsetYCm = static_cast<float>(InCameraView.OffCenterProjectionOffset.Y) * EffectiveSensorHeightCm * 0.5f;

		for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
		{
			LocalSensorCorners[CornerIdx].X += SensorOffsetXCm;
			LocalSensorCorners[CornerIdx].Y += SensorOffsetYCm;
		}
	}

	// Compute focus plane corners in view space
	FVector FocusCorners_ViewSpace[4];
	AccumulationDOFMath::ComputeFocusPlaneCorners(LocalSensorCorners, FocusDistanceCm, FocusCorners_ViewSpace);

	// Sample camera position
	const FVector SampleLocation = CameraState.CameraLocation + RightVec * ApertureOffsetCm.X + UpVec * ApertureOffsetCm.Y;

	// Transform focus corners to world space
	FVector FocusCorners_World[4];
	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		FocusCorners_World[Idx] = CameraState.CameraLocation +
			RightVec * FocusCorners_ViewSpace[Idx].X +
			UpVec * FocusCorners_ViewSpace[Idx].Y +
			ForwardVec * FocusCorners_ViewSpace[Idx].Z;
	}

	// Transform to sample camera's view space
	FVector FocusCorners_SampleView[4];
	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		FVector WorldOffset = FocusCorners_World[Idx] - SampleLocation;
		FocusCorners_SampleView[Idx] = FVector(
			FVector::DotProduct(WorldOffset, RightVec),
			FVector::DotProduct(WorldOffset, UpVec),
			FVector::DotProduct(WorldOffset, ForwardVec)
		);
	}

	// Compute frustum bounds

	float Left   = FLT_MAX;
	float Right  = -FLT_MAX;
	float Bottom = FLT_MAX;
	float Top    = -FLT_MAX;

	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		const float Z = FocusCorners_SampleView[Idx].Z;

		if (FMath::Abs(Z) < KINDA_SMALL_NUMBER)
		{
			return false;
		}

		const float Scale = NearCm / Z;
		const float X = FocusCorners_SampleView[Idx].X * Scale;
		const float Y = FocusCorners_SampleView[Idx].Y * Scale;

		Left   = FMath::Min(Left, X);
		Right  = FMath::Max(Right, X);
		Bottom = FMath::Min(Bottom, Y);
		Top    = FMath::Max(Top, Y);
	}

	// Note: Jitter not applied here but is applied later using the sample state ProjectionMatrixJitterAmount.

	OutMatrix = AccumulationDOFMath::BuildOffAxisProjectionMatrix(Left, Right, Bottom, Top, NearCm, FarCm);

	// BuildOffAxisProjectionMatrix returns Identity for degenerate bounds.
	if (OutMatrix.Equals(FMatrix::Identity))
	{
		return false;
	}

	return true;
}

void UApertureSampler::AccumulateSampleInternal(
	int32 SampleIndex,
	const FVector2f& ApertureOffsetCm,
	const FVector3f& SpectralWeight
)
{
	if (!SampleRT || !AccumA || !AccumB || !PrefilterRT)
	{
		return;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();

	if (!CineCam)
	{
		return;
	}

	// Apply pre-median filter to sample if enabled
	const bool bPreMedianEnabled = CVarAccumulationDOFPreMedianFilterEnable.GetValueOnGameThread() > 0;
	if (bPreMedianEnabled)
	{
		UTextureRenderTarget2D* SampleObj = SampleRT.Get();
		UTextureRenderTarget2D* PrefilterObj = PrefilterRT.Get();
		ENQUEUE_RENDER_COMMAND(AccumulationDOFPreMedianFilter)(
			[SampleObj, PrefilterObj](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* SampleResource = SampleObj ? SampleObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* PrefilterResource = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
				if (SampleResource && PrefilterResource)
				{
					AccumulationDOF::ApplyMedianFilter(RHICmdList, SampleResource, PrefilterResource);
				}
			}
		);
	}

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	UTextureRenderTarget2D* PrevAccumObj = AccumA.Get();
	UTextureRenderTarget2D* SampleObj = bPreMedianEnabled ? PrefilterRT.Get() : SampleRT.Get();
	UTextureRenderTarget2D* OutputObj = AccumB.Get();

	// Extract camera parameters
	const AccumulationDOFUtils::FCameraParams CamParams = AccumulationDOFUtils::ExtractCameraParams(CineCam);

	// Get bokeh texture resources
	FRHITexture* BokehTextureRHI = nullptr;
	FRHISamplerState* BokehSamplerRHI = nullptr;
	bool bBokehEnabled = false;

	if (Config.BokehTexture && Config.bEnableBokehTexture)
	{
		if (FTextureResource* BokehResource = Config.BokehTexture->GetResource())
		{
			BokehTextureRHI = BokehResource->TextureRHI;
			BokehSamplerRHI = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			bBokehEnabled = (BokehTextureRHI != nullptr);
		}
	}

	// Compute bokeh shape
	const AccumulationDOFUtils::FBokehShapeParams BokehShape = AccumulationDOFUtils::ComputeBokehShape(
		CamParams.FStop, CamParams.MinFstop, CamParams.BladeCountRaw
	);

	// Build accumulation parameters
	FAccumulateSampleParams AccumParams;

	AccumParams.ApertureOffsetCm = ApertureOffsetCm;
	AccumParams.SpectralWeight = SpectralWeight;

	AccumParams.ApertureRadiusCm = CamParams.ApertureRadiusCm;
	AccumParams.SqueezeFactor = CamParams.SqueezeFactor;
	AccumParams.Petzval = CamParams.Petzval;
	AccumParams.PetzvalFalloffPower = CamParams.PetzvalFalloffPower;
	AccumParams.PetzvalExclusionBoxExtents = CamParams.PetzvalExclusionBoxExtents;
	AccumParams.PetzvalExclusionBoxRadius = CamParams.PetzvalExclusionBoxRadius;
	AccumParams.BarrelRadiusCm = CamParams.BarrelRadiusCm;
	AccumParams.BarrelLengthCm = CamParams.BarrelLengthCm;
	AccumParams.FocalLengthCm = CamParams.FocalLengthCm;
	AccumParams.FocusDistanceCm = CamParams.FocusDistanceCm;
	AccumParams.SensorHalfSizeCm = FVector2f(CamParams.SensorWidthCm * 0.5f, CamParams.SensorHeightCm * 0.5f);
	AccumParams.ComaStrength = Config.ComaAberration * 0.1f;

	AccumParams.bBokehEnabled = bBokehEnabled;
	AccumParams.BokehWeightChannel = static_cast<int32>(Config.WeightChannel);
	AccumParams.bUseTint = Config.TintStrength > KINDA_SMALL_NUMBER;
	AccumParams.TintStrength = Config.TintStrength;
	AccumParams.BokehEdgeSoftness = Config.BokehEdgeSoftness;

	AccumParams.BokehShape = BokehShape.BokehShape;
	AccumParams.DiaphragmBladeCount = BokehShape.DiaphragmBladeCount;
	AccumParams.DiaphragmRotationRad = BokehShape.DiaphragmRotationRad;
	AccumParams.CocRadiusToCircumscribedRadius = BokehShape.CocRadiusToCircumscribedRadius;
	AccumParams.CocRadiusToIncircleRadius = BokehShape.CocRadiusToIncircleRadius;
	AccumParams.DiaphragmBladeRadius = BokehShape.DiaphragmBladeRadius;
	AccumParams.DiaphragmBladeCenterOffset = BokehShape.DiaphragmBladeCenterOffset;

	// AA reconstruction weight and bilinear splatting, but skip for single sample which allows direct comparison with MRQ deferred pass.

	const bool bJitterAAAllowed = !!CVarAccumulationDOFJitterAAAllow.GetValueOnAnyThread();
	const bool bMultipleSamples = Progress.ActualNumSamples > 1;
	const EAAFilter AAFilter = static_cast<EAAFilter>(CVarAccumulationDOFJitterAAFilter.GetValueOnAnyThread());

	if (Config.bUseJitterAA && bJitterAAAllowed && bMultipleSamples)
	{
		const FVector2f JitterOffset = GetJitterOffset(SampleIndex);
		switch (AAFilter)
		{
		case EAAFilter::Mitchell:
			AccumParams.AntiAliasingWeight = AccumulationDOFMath::MitchellNetravali2D(JitterOffset);
			break;
		case EAAFilter::Box:
		default:
			AccumParams.AntiAliasingWeight = 1.0f;
			break;
		}

		// Bilinear subpixel splatting via gather.

		const bool bBilinearSplatEnabled = !!CVarAccumulationDOFJitterAABilinearSplat.GetValueOnAnyThread();

		AccumParams.SubpixelOffset = bBilinearSplatEnabled
			? FVector2f(0.5f, 0.5f) + JitterOffset
			: FVector2f(0.5f, 0.5f);
	}
	else
	{
		AccumParams.AntiAliasingWeight = 1.0f;
		AccumParams.SubpixelOffset = FVector2f(0.5f, 0.5f);
	}

	// Texel size for bilinear splatting UV offset
	AccumParams.TexelSize = FVector2f(
		1.0f / FMath::Max(1.0f, static_cast<float>(Config.Resolution.X)),
		1.0f / FMath::Max(1.0f, static_cast<float>(Config.Resolution.Y))
	);

	// Execute accumulation
	ENQUEUE_RENDER_COMMAND(AccumulationDOFAccumulate)(
		[PrevAccumObj, SampleObj, OutputObj, BokehTextureRHI, BokehSamplerRHI, AccumParams]
		(FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* PrevAccumResource = PrevAccumObj ? PrevAccumObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* SampleResource = SampleObj ? SampleObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* OutputResource = OutputObj ? OutputObj->GetRenderTargetResource() : nullptr;
			if (!PrevAccumResource || !SampleResource || !OutputResource)
			{
				return;
			}
			AccumulationDOF::AccumulateSample(
				RHICmdList,
				PrevAccumResource,
				SampleResource,
				OutputResource,
				BokehTextureRHI,
				BokehSamplerRHI,
				AccumParams
			);
		}
	);

	// Swap ping-pong buffers
	Swap(AccumA, AccumB);
}

void UApertureSampler::AccumulateExternalSample(
	int32 SampleIndex,
	const FVector2f& ApertureOffsetCm,
	const FVector3f& SpectralWeight
)
{
	AccumulateSampleInternal(SampleIndex, ApertureOffsetCm, SpectralWeight);
}

void UApertureSampler::FinalizeAccumulation()
{
	NormalizeAccumulation(static_cast<float>(Progress.ActualNumSamples));
	Progress.bComplete = true;
	bHasValidAccumulatedData = true;
}

void UApertureSampler::NormalizeAccumulation(float TotalWeightSum)
{
	if (!AccumA || !PrefilterRT)
	{
		return;
	}

	if (Progress.ActualNumSamples <= 0 || TotalWeightSum <= 0.0f)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "NormalizeAccumulation skipped: ActualNumSamples=%d TotalWeightSum=%f",
			Progress.ActualNumSamples, TotalWeightSum);
		return;
	}

	const float InvWeightSum = 1.0f / FMath::Max(TotalWeightSum, 1e-6f);

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	UTextureRenderTarget2D* AccumAObj = AccumA.Get();
	UTextureRenderTarget2D* PrefilterObj = PrefilterRT.Get();
	UTextureRenderTarget2D* AccumBObj = AccumB.Get();

	// Normalize to PrefilterRT
	ENQUEUE_RENDER_COMMAND(AccumulationDOFNormalize)(
		[AccumAObj, PrefilterObj, InvWeightSum](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* AccumA_RT = AccumAObj ? AccumAObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* Normalized_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
			if (AccumA_RT && Normalized_RT)
			{
				AccumulationDOF::NormalizeAccumulation(RHICmdList, AccumA_RT, Normalized_RT, InvWeightSum);
			}
		}
	);

	bHasValidAccumulatedData = true;

	// Apply spectral lateral CA if enabled
	ApplySpectralLateralCA();

	// Apply median filter if enabled
	ApplyMedianFilter();

	// Copy final result to AccumA
	const bool bMedianEnabled = Config.bEnableMedianFilter && AccumBObj && Config.MedianFilterIterations > 0
		&& CVarAccumulationDOFPostMedianFilterEnable.GetValueOnGameThread() > 0;
	const int32 MedianIterations = bMedianEnabled ? FMath::Clamp(Config.MedianFilterIterations, 1, 10) : 0;

	ENQUEUE_RENDER_COMMAND(AccumulationDOFCopyNormalized)(
		[AccumAObj, PrefilterObj, AccumBObj, bMedianEnabled, MedianIterations](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* FinalAccumA_RT = AccumAObj ? AccumAObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* FilteredSource_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;

			if (bMedianEnabled)
			{
				FTextureRenderTargetResource* AccumB_RT = AccumBObj ? AccumBObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* Normalized_RT = FilteredSource_RT;
				FilteredSource_RT = (MedianIterations % 2 == 1) ? AccumB_RT : Normalized_RT;
			}

			if (FilteredSource_RT && FinalAccumA_RT && FilteredSource_RT != FinalAccumA_RT)
			{
				AccumulationDOF::CopyTexture(RHICmdList, FilteredSource_RT, FinalAccumA_RT);
			}
		}
	);
}

void UApertureSampler::ApplySpectralLateralCA()
{
	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		return;
	}

	const float LateralCAIntensity = CameraState.SceneFringeIntensity;
	const bool bDoSpectralCA = Config.bSpectralLateralChromaticAberration && (LateralCAIntensity > UE_KINDA_SMALL_NUMBER);

	if (!bDoSpectralCA || !PrefilterRT || !AccumB)
	{
		return;
	}

	const float StartOffset = CineCam->PostProcessSettings.bOverride_ChromaticAberrationStartOffset
		? CineCam->PostProcessSettings.ChromaticAberrationStartOffset
		: 0.0f;

	constexpr float EngineCAScale = 0.01029f;
	const float Multiplier = (StartOffset < 1.0f - UE_KINDA_SMALL_NUMBER) ? (1.0f / (1.0f - StartOffset)) : 1.0f;
	const float IntensityUV = LateralCAIntensity * EngineCAScale * Multiplier;

	const FVector2f Center(0.5f, 0.5f);

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	UTextureRenderTarget2D* PrefilterObj = PrefilterRT.Get();
	UTextureRenderTarget2D* AccumBObj = AccumB.Get();

	// Apply spectral lateral CA (PrefilterRT -> AccumB)
	ENQUEUE_RENDER_COMMAND(AccumulationDOFSpectralLateralCA)(
		[PrefilterObj, AccumBObj, IntensityUV, StartOffset, Center](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* Normalized_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* AccumB_RT = AccumBObj ? AccumBObj->GetRenderTargetResource() : nullptr;
			if (Normalized_RT && AccumB_RT)
			{
				AccumulationDOF::ApplyLateralCA(RHICmdList, Normalized_RT, AccumB_RT, IntensityUV, StartOffset, Center);
			}
		}
	);

	// Copy result back to PrefilterRT
	ENQUEUE_RENDER_COMMAND(AccumulationDOFCopyCAResult)(
		[AccumBObj, PrefilterObj](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* AccumB_RT = AccumBObj ? AccumBObj->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* Normalized_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
			if (AccumB_RT && Normalized_RT)
			{
				AccumulationDOF::CopyTexture(RHICmdList, AccumB_RT, Normalized_RT);
			}
		}
	);
}

void UApertureSampler::ApplyMedianFilter()
{
	if (!Config.bEnableMedianFilter || Config.MedianFilterIterations <= 0 || !PrefilterRT || !AccumB
		|| CVarAccumulationDOFPostMedianFilterEnable.GetValueOnGameThread() <= 0)
	{
		return;
	}

	const int32 NumIterations = FMath::Clamp(Config.MedianFilterIterations, 1, 10);

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	UTextureRenderTarget2D* PrefilterObj = PrefilterRT.Get();
	UTextureRenderTarget2D* AccumBObj = AccumB.Get();

	for (int32 Iter = 0; Iter < NumIterations; ++Iter)
	{
		const bool bEvenIteration = (Iter % 2 == 0);

		ENQUEUE_RENDER_COMMAND(AccumulationDOFMedianFilter)(
			[PrefilterObj, AccumBObj, bEvenIteration](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* Normalized_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* AccumB_RT = AccumBObj ? AccumBObj->GetRenderTargetResource() : nullptr;
				if (!Normalized_RT || !AccumB_RT)
				{
					return;
				}
				FTextureRenderTargetResource* SrcRT = bEvenIteration ? Normalized_RT : AccumB_RT;
				FTextureRenderTargetResource* DstRT = bEvenIteration ? AccumB_RT : Normalized_RT;
				AccumulationDOF::ApplyMedianFilter(RHICmdList, SrcRT, DstRT);
			}
		);
	}
}

UTextureRenderTarget2D* UApertureSampler::PreparePreviewTexture()
{
	if (!AccumA || !PrefilterRT)
	{
		return nullptr;
	}

	if (Progress.CurrentIteration == 0)
	{
		return nullptr;
	}

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	UTextureRenderTarget2D* AccumAObj = AccumA.Get();
	UTextureRenderTarget2D* PrefilterObj = PrefilterRT.Get();

	// Always output to PrefilterRT to avoid timing issues when switching textures.
	if (Progress.bComplete)
	{
		// AccumA already has normalized result - just copy it
		ENQUEUE_RENDER_COMMAND(AccumulationDOFPreparePreview)(
			[AccumAObj, PrefilterObj](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* AccumA_RT = AccumAObj ? AccumAObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* Prefilter_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
				if (AccumA_RT && Prefilter_RT)
				{
					AccumulationDOF::CopyTexture(RHICmdList, AccumA_RT, Prefilter_RT);
				}
			}
		);
	}
	else
	{
		// Normalize partial accumulation (AccumA has weighted sum, not yet normalized)
		const int32 EffectiveBands = GetEffectiveBands();
		const int32 SampleCount = FMath::Max(1, Progress.CurrentIteration / FMath::Max(1, EffectiveBands));
		const float InvWeight = 1.0f / static_cast<float>(SampleCount);

		ENQUEUE_RENDER_COMMAND(AccumulationDOFPreparePreview)(
			[AccumAObj, PrefilterObj, InvWeight](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* AccumA_RT = AccumAObj ? AccumAObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* Prefilter_RT = PrefilterObj ? PrefilterObj->GetRenderTargetResource() : nullptr;
				if (AccumA_RT && Prefilter_RT)
				{
					AccumulationDOF::NormalizeAccumulation(RHICmdList, AccumA_RT, Prefilter_RT, InvWeight);
				}
			}
		);

		ApplySpectralLateralCA();
	}

	return PrefilterRT;
}

void UApertureSampler::CopyToOutput(UTextureRenderTarget2D* OutputRT, bool bDrawProgress)
{
	if (!OutputRT || !AccumA)
	{
		return;
	}

	// Resize if needed
	if (OutputRT->SizeX != Config.Resolution.X || OutputRT->SizeY != Config.Resolution.Y)
	{
		OutputRT->ResizeTarget(Config.Resolution.X, Config.Resolution.Y);
	}

	// Prepare the preview texture (normalizes if in progress)
	UTextureRenderTarget2D* SourceRT = PreparePreviewTexture();
	if (!SourceRT)
	{
		return;
	}

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	const float ProgressFraction = Progress.GetProgressFraction();
	const bool bShowProgress = bDrawProgress && !Progress.bComplete;

	ENQUEUE_RENDER_COMMAND(CopyToOutput)(
		[SourceRT, OutputRT, ProgressFraction, bShowProgress](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* SrcResource = SourceRT ? SourceRT->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* DstResource = OutputRT ? OutputRT->GetRenderTargetResource() : nullptr;
			if (!SrcResource || !DstResource)
			{
				return;
			}
			if (bShowProgress)
			{
				AccumulationDOF::CopyWithProgressBar(RHICmdList, SrcResource, DstResource, ProgressFraction);
			}
			else
			{
				AccumulationDOF::CopyTexture(RHICmdList, SrcResource, DstResource);
			}
		}
	);
}

void UApertureSampler::CopyToOutputCropped(
	UTextureRenderTarget2D* OutputRT,
	const FVector2f& SourceUVMin,
	const FVector2f& SourceUVMax,
	bool bDrawProgress
)
{
	if (!OutputRT || !AccumA)
	{
		return;
	}

	// Note: OutputRT should already be sized appropriately by the caller
	// We do NOT resize it here (unlike CopyToOutput)

	// Prepare the preview texture (normalizes if in progress)
	UTextureRenderTarget2D* SourceRT = PreparePreviewTexture();
	if (!SourceRT)
	{
		return;
	}

	// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
	const FVector2f UVMin = SourceUVMin;
	const FVector2f UVMax = SourceUVMax;

	ENQUEUE_RENDER_COMMAND(CopyToOutputCropped)(
		[SourceRT, OutputRT, UVMin, UVMax](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTargetResource* SrcResource = SourceRT ? SourceRT->GetRenderTargetResource() : nullptr;
			FTextureRenderTargetResource* DstResource = OutputRT ? OutputRT->GetRenderTargetResource() : nullptr;
			if (SrcResource && DstResource)
			{
				AccumulationDOF::CopyCropped(RHICmdList, SrcResource, DstResource, UVMin, UVMax);
			}
		}
	);
}

bool UApertureSampler::RenderWithPostProcessing(
	UTextureRenderTarget2D* OutputRT,
	bool bAllowSceneFringe,
	float ProgressBarFraction,
	float Overscan
)
{
	if (!bIsInitialized || !Renderer || !OutputRT)
	{
		return false;
	}

	UCineCameraComponent* CineCam = CameraState.CineCameraComponent.Get();
	if (!CineCam)
	{
		return false;
	}

	// Prepare the preview texture (normalizes if in progress)
	UTextureRenderTarget2D* SourceRT = PreparePreviewTexture();
	if (!SourceRT)
	{
		return false;
	}

	// Determine if we need overscan cropping
	const bool bNeedsCropping = Overscan > 0.0f;
	UTextureRenderTarget2D* PPTargetRT = OutputRT;

	if (bNeedsCropping)
	{
		// Create or resize PostProcessRT for intermediate result at full internal resolution
		if (!PostProcessRT)
		{
			PostProcessRT = NewObject<UTextureRenderTarget2D>(this, TEXT("PostProcessRT"));
			PostProcessRT->RenderTargetFormat = RTF_RGBA16f;
			PostProcessRT->ClearColor = FLinearColor::Black;
			PostProcessRT->InitAutoFormat(Config.Resolution.X, Config.Resolution.Y);
			PostProcessRT->UpdateResourceImmediate(true);
		}
		else if (PostProcessRT->SizeX != Config.Resolution.X || PostProcessRT->SizeY != Config.Resolution.Y)
		{
			PostProcessRT->ResizeTarget(Config.Resolution.X, Config.Resolution.Y);
		}
		PPTargetRT = PostProcessRT;
	}
	else
	{
		// No cropping - resize output to internal resolution and render directly
		if (OutputRT->SizeX != Config.Resolution.X || OutputRT->SizeY != Config.Resolution.Y)
		{
			OutputRT->ResizeTarget(Config.Resolution.X, Config.Resolution.Y);
		}

		// Release PostProcessRT when not needed
		if (PostProcessRT)
		{
			PostProcessRT->ReleaseResource();
			PostProcessRT = nullptr;
		}
	}

	// Motion blur is disabled for component preview.
	const bool bEnableMotionBlur = false;

	const bool bSuccess = Renderer->RenderWithInjection(
		SourceRT,
		PPTargetRT,
		CineCam,
		Config.PostProcessOutputFormat,
		bAllowSceneFringe,
		bEnableMotionBlur,
		false,                   // bWorldIsPaused
		ProgressBarFraction
	);

	// Crop from PostProcessRT to OutputRT if needed
	if (bSuccess && bNeedsCropping)
	{
		const float OverscanScale = 1.0f + Overscan;
		const float CropMargin = (OverscanScale - 1.0f) / (2.0f * OverscanScale);
		const FVector2f SourceUVMin(CropMargin, CropMargin);
		const FVector2f SourceUVMax(1.0f - CropMargin, 1.0f - CropMargin);

		// Capture UTextureRenderTarget2D pointers and resolve resources on render thread
		UTextureRenderTarget2D* PostProcessObj = PostProcessRT.Get();
		ENQUEUE_RENDER_COMMAND(AccumulationDOFCropPostProcess)(
			[PostProcessObj, OutputRT, SourceUVMin, SourceUVMax](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* PostProcessResource = PostProcessObj ? PostProcessObj->GetRenderTargetResource() : nullptr;
				FTextureRenderTargetResource* OutputResource = OutputRT ? OutputRT->GetRenderTargetResource() : nullptr;
				if (PostProcessResource && OutputResource)
				{
					AccumulationDOF::CopyCropped(RHICmdList, PostProcessResource, OutputResource, SourceUVMin, SourceUVMax);
				}
			}
		);
	}

	return bSuccess;
}

void UApertureSampler::SaveSampleImage(int32 SampleIndex, const FString& Directory, const FString& Prefix)
{
	if (!SampleRT)
	{
		return;
	}

	FString FullDirectory = FPaths::IsRelative(Directory)
		? FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / Directory)
		: Directory;

	IFileManager::Get().MakeDirectory(*FullDirectory, true);

	FString Filename = FString::Printf(TEXT("%s_Sample_%04d.exr"), *Prefix, SampleIndex);

	UWorld* World = Config.World.Get();
	if (World)
	{
		UKismetRenderingLibrary::ExportRenderTarget(World, SampleRT, FullDirectory, Filename);
	}
}

void UApertureSampler::SaveFinalImage(const FString& Directory, const FString& Prefix)
{
	if (!AccumA)
	{
		return;
	}

	// Ensure GPU work is complete before reading back for save
	FlushRenderingCommands();

	FString FullDirectory = FPaths::IsRelative(Directory)
		? FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / Directory)
		: Directory;

	IFileManager::Get().MakeDirectory(*FullDirectory, true);

	FString FilenamePrePP = FString::Printf(TEXT("%s_Final_PrePP.exr"), *Prefix);

	UWorld* World = Config.World.Get();
	if (World)
	{
		UKismetRenderingLibrary::ExportRenderTarget(World, AccumA, FullDirectory, FilenamePrePP);
	}
}

FVector2f UApertureSampler::GetApertureOffset(int32 SampleIndex) const
{
	const bool bApertureJitterAllowed = !!CVarAccumulationDOFApertureJitterAllowed.GetValueOnAnyThread();
	if (bApertureJitterAllowed && SampleIndex >= 0 && SampleIndex < ApertureOffsetsCm.Num())
	{
		return ApertureOffsetsCm[SampleIndex];
	}
	return FVector2f::ZeroVector;
}

FVector2f UApertureSampler::GetJitterOffset(int32 SampleIndex) const
{
	const EJitterSequence JitterSequence = static_cast<EJitterSequence>(CVarAccumulationDOFJitterAASequence.GetValueOnAnyThread());

	switch (JitterSequence)
	{
	case EJitterSequence::Sobol:
		return AccumulationDOFMath::GenerateSobolJitter(SampleIndex);

	case EJitterSequence::Halton:
	default:
		return AccumulationDOFMath::GenerateHaltonJitter(SampleIndex);
	}
}

FVector2D UApertureSampler::GetJitterForProjectionMatrix(int32 SampleIndex, const FIntPoint& Resolution) const
{
	const bool bJitterAAAllowed = !!CVarAccumulationDOFJitterAAAllow.GetValueOnAnyThread();

	if (!Config.bUseJitterAA || !bJitterAAAllowed || Resolution.X <= 0 || Resolution.Y <= 0)
	{
		return FVector2D::ZeroVector;
	}

	// No jitter for single sample
	if (Progress.ActualNumSamples <= 1)
	{
		return FVector2D::ZeroVector;
	}

	const FVector2f JitterNorm = GetJitterOffset(SampleIndex);

	// Convert to clip space.
	return FVector2D(
		JitterNorm.X * 2.0 / Resolution.X,
		JitterNorm.Y * -2.0 / Resolution.Y
	);
}

float UApertureSampler::ComputeSpectralFocusDistance(float NominalFocusCm, float NormalizedSpectralPosition) const
{
	// Blue (0) focuses closer, red (1) focuses farther
	float SpectralOffset = -1.0f + 2.0f * NormalizedSpectralPosition;
	SpectralOffset *= Config.AxialChromaticAberrationIntensity * 0.01f;
	const float Result = NominalFocusCm * (1.0f + SpectralOffset);
	return FMath::Max(Result, UE_KINDA_SMALL_NUMBER);
}

FVector3f UApertureSampler::ComputeSpectralWeight(float NormalizedSpectralPosition) const
{
	const float IndexFloat = FMath::Clamp(NormalizedSpectralPosition, 0.0f, 1.0f) * 31.0f;
	const int32 Index0 = FMath::FloorToInt32(IndexFloat);
	const int32 Index1 = FMath::Min(Index0 + 1, 31);
	const float T = IndexFloat - static_cast<float>(Index0);

	return FVector3f(
		FMath::Lerp(SpectralLUT[Index0][0], SpectralLUT[Index1][0], T),
		FMath::Lerp(SpectralLUT[Index0][1], SpectralLUT[Index1][1], T),
		FMath::Lerp(SpectralLUT[Index0][2], SpectralLUT[Index1][2], T)
	);
}

int32 UApertureSampler::GetEffectiveBands() const
{
	const bool bHasAxialCA = Config.AxialChromaticAberrationIntensity > UE_KINDA_SMALL_NUMBER;
	return bHasAxialCA ? FMath::Min(FMath::Clamp(Config.AxialChromaticAberrationNumBands, 3, 19), Progress.ActualNumSamples) : 1;
}

int32 UApertureSampler::GetTotalIterations() const
{
	const int32 EffectiveBands = GetEffectiveBands();
	return Progress.ActualNumSamples * EffectiveBands;
}

void UApertureSampler::NotifyProgress()
{
	if (OnProgressDelegate.IsBound())
	{
		OnProgressDelegate.Execute(Progress);
	}
}
