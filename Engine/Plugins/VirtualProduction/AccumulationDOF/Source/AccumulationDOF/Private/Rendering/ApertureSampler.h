// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "AccumulationDOFTypes.h"

#include "Engine/EngineBaseTypes.h"
#include "Engine/Scene.h"

#include "ApertureSampler.generated.h"

class UApertureSamplingRenderer;
class UCineCameraComponent;
class UTexture2D;
class UTextureRenderTarget2D;
class UWorld;
class FTextureRenderTargetResource;
class FSceneViewStateInterface;
struct FMinimalViewInfo;

namespace AccumulationDOF
{

struct FApertureSampleParams;

/**
 * Rendering mode for aperture sampling.
 */
enum class EApertureSamplerMode : uint8
{
	/** Render all samples in one blocking call. */
	OneShot,

	/**
	 * Spread samples across multiple frames. Used in preview mode to not block the Engine.
	 * However if the scene has motion then they will smear the accumulation.
	 */
	Amortized
};

/**
 * Configuration for aperture sampling.
 */
struct FApertureSamplerConfig
{
	/** Output resolution */
	FIntPoint Resolution = FIntPoint(1920, 1080);

	/** Reference to the world for rendering. */
	TWeakObjectPtr<UWorld> World;

	//
	// Sampling Configuration
	//

	/** Number of aperture samples requested. Actual count may differ for Ring pattern. */
	int32 NumSamples = 256;

	/** Sampling pattern (Halton, Hexaweb, Vogel). */
	EApertureSamplingPattern SamplingPattern = EApertureSamplingPattern::Hexaweb;

	/** Rendering mode (OneShot vs Amortized). */
	EApertureSamplerMode Mode = EApertureSamplerMode::OneShot;

	/** Number of samples to render per frame in amortized mode */
	int32 SamplesPerFrame = 1;

	//
	// Anti-aliasing
	//

	/** Use Halton jitter AA instead of per-sample TAA. */
	bool bUseJitterAA = true;

	//
	// DOF splats (helps fill gaps but blurs bokeh edges)
	//

	/**
	 * Fraction of main aperture diameter for DOF splat size (0.125 = 1/8th, 0 = disabled).
	 * Smaller values fill less but produce sharper bokeh edges.
	 */
	float DOFSplatSize = 0.125f;

	//
	// Chromatic Aberration
	//

	/** Axial CA intensity as percentage of focus distance. */
	float AxialChromaticAberrationIntensity = 0.0f;

	/** Number of spectral bands for axial CA (3-19). */
	int32 AxialChromaticAberrationNumBands = 6;

	/** Multi-band spectral lateral CA instead of engine's simple RGB split. */
	bool bSpectralLateralChromaticAberration = true;

	//
	// Monochromatic Aberrations
	//

	/** Primary spherical aberration coefficient (Seidel W040) in cm. */
	float SphericalAberration = 0.0f;

	/** Coma aberration strength (Seidel W131). Normalized and scaled by 0.1 before passing to shader. */
	float ComaAberration = 0.0f;

	//
	// Bokeh Settings
	//

	/** Custom bokeh texture for aperture weighting. */
	TObjectPtr<UTexture2D> BokehTexture = nullptr;

	/** Enable bokeh texture weighting. */
	bool bEnableBokehTexture = true;

	/** Which channel to use for bokeh weight (luminance or alpha channel). */
	EBokehWeightChannel WeightChannel = EBokehWeightChannel::Luminance;

	/** Strength of bokeh tint (0-1). */
	float TintStrength = 0.0f;

	/** Bokeh edge softness (0 = hard, 1 = very soft). */
	float BokehEdgeSoftness = 0.15f;

	//
	// Denoising
	//

	/** Enable median filter after accumulation. */
	bool bEnableMedianFilter = true;

	/** Number of median filter iterations. */
	int32 MedianFilterIterations = 1;

	//
	// Temporal History
	//

	/** Controls which samples update temporal history. */
	ETemporalHistoryMode TemporalHistoryMode = ETemporalHistoryMode::LastSampleOnly;

	//
	// Post-Processing
	//

	/** Output format for post-processed result. */
	TEnumAsByte<ESceneCaptureSource> PostProcessOutputFormat = ESceneCaptureSource::SCS_FinalToneCurveHDR;

	//
	// Overscan
	//

	/**
	 * Overscan as a normalized fraction in [0,1] range (0.0 = none, 1.0 = 100% overscan).
	 */
	float OverscanFraction = 0.0f;

	/**
	 * Screen percentage fraction (0.1 to 4.0, where 1.0 = 100%).
	 * Negative value means use r.ScreenPercentage
	 */
	float ScreenPercentage = -1.0f;

	//
	// View Mode
	//

	/** View mode for rendering (Lit, Detail Lighting, Lighting Only, Reflections). */
	EViewModeIndex ViewModeIndex = VMI_Lit;

	//
	// Debug Output
	//

	/** Save intermediate sample images to disk. */
	bool bSaveIntermediateSamples = false;

	/** Save final accumulated image to disk. */
	bool bSaveFinalAccumulation = false;

	/** Output directory for saved images. */
	FString OutputDirectory;

	/** Output filename prefix. */
	FString OutputFilenamePrefix;

	//
	// Exposure
	//

	/** Optional view state for exposure calculations (shares exposure history with another view) */
	FSceneViewStateInterface* ExposureViewState = nullptr;
};

/**
 * Per-frame camera state needed for rendering.
 * Updated each frame or when camera parameters change.
 */
struct FApertureSamplerCameraState
{
	/** CineCameraComponent to read parameters from. */
	TWeakObjectPtr<UCineCameraComponent> CineCameraComponent;

	/** Camera world location. */
	FVector CameraLocation = FVector::ZeroVector;

	/** Camera world rotation. */
	FRotator CameraRotation = FRotator::ZeroRotator;

	/** Captured SceneFringeIntensity from blended post-process settings. Cached because we replace lateral CA */
	float SceneFringeIntensity = 0.0f;

	/** Initialization state */
	bool bInitialized = false;
};

/**
 * Progress information for the sampling operation.
 */
struct FApertureSamplerProgress
{
	/** Current sample index being rendered */
	int32 CurrentIteration = 0;

	/** Total number of iterations to render */
	int32 TotalIterations = 0;

	/**
	 * Actual number of aperture samples.
	 */
	int32 ActualNumSamples = 0;

	/** Whether accumulation is complete or not. */
	bool bComplete = false;

	/** Whether the operation was cancelled by user. */
	bool bWasCancelled = false;

	/** Progress as a fraction [0, 1]. */
	float GetProgressFraction() const
	{
		return TotalIterations > 0 ? static_cast<float>(CurrentIteration) / static_cast<float>(TotalIterations) : 0.0f;
	}

	/** For progress display */
	FString GetProgressString() const
	{
		if (bComplete)
		{
			return FString::Printf(TEXT("%d/%d (Complete)"), TotalIterations, TotalIterations);
		}
		return FString::Printf(TEXT("%d/%d"), CurrentIteration, TotalIterations);
	}
};

/** Delegate for progress updates during sampling. */
DECLARE_DELEGATE_OneParam(FOnApertureSamplerProgress, const FApertureSamplerProgress&);

/** Delegate for completion notification.*/
DECLARE_DELEGATE_OneParam(FOnApertureSamplerComplete, UTextureRenderTarget2D* /*AccumulatedResult*/);

} // namespace AccumulationDOF

/**
 * Encapsulates the aperture sampling pipeline.
 *
 * This class manages the entire lifecycle of aperture-sampled rendering.
 *
 * Can render one-shot (blocking) or amortized.
 */
UCLASS()
class ACCUMULATIONDOF_API UApertureSampler : public UObject
{
	GENERATED_BODY()

public:
	UApertureSampler();

	/**
	 * Initialize the sampler with configuration and camera state.
	 * Allocates render targets and generates aperture sample positions.
	 *
	 * @param InConfig      - Configuration for sampling behavior
	 * @param InCameraState - Initial camera state
	 *
	 * @return true if initialization succeeded
	 */
	bool Initialize(const AccumulationDOF::FApertureSamplerConfig& InConfig, const AccumulationDOF::FApertureSamplerCameraState& InCameraState);

	/**
	 * Check if the sampler is properly initialized and ready to render.
	 */
	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	/**
	 * Release all resources and reset state.
	 */
	void Shutdown();

	/**
	 * Update camera state for the current frame.
	 * Call this before RenderAmortizedSamples() in amortized mode.
	 * Detects significant changes and resets accumulation if needed.
	 *
	 * @param InCameraState - New camera state
	 *
	 * @return true if accumulation was reset due to significant changes
	 */
	bool UpdateCameraState(const AccumulationDOF::FApertureSamplerCameraState& InCameraState);

	/**
	 * Force-reset accumulation (use when user clicks on Recapture button because the scene or a setting changed).
	 */
	void ResetAccumulation();

	/**
	 * Get the current configuration.
	 */
	const AccumulationDOF::FApertureSamplerConfig& GetConfig() const
	{
		return Config;
	}

	/**
	 * Get the actual number of samples (may differ from requested if the pattern requires it)
	 */
	int32 GetActualNumSamples() const
	{
		return Progress.ActualNumSamples;
	}

	/**
	 * Render all aperture samples in a blocking fashion.
	 *
	 * @return true if rendering succeeded
	 */
	bool RenderAllSamples();

	//
	// Amortized Rendering
	//

	/**
	 * Render one frame's worth of samples (SamplesPerFrame).
	 *
	 * @return true if samples were rendered (false if complete or error)
	 */
	bool RenderAmortizedSamples();

	/**
	 * @return true if amortized accumulation is complete.
	 */
	bool IsComplete() const
	{
		return Progress.bComplete;
	}

	//
	// Results Access
	//

	/**
	 * Get the final accumulated result (after normalization and post-processing).
	 * Valid after RenderAllSamples() or when IsComplete() returns true.
	 *
	 * @return Accumulated render target (AccumA), or nullptr if not ready
	 */
	UTextureRenderTarget2D* GetAccumulatedResult() const
	{
		return AccumA;
	}

	/**
	 * Prepare and return the texture ready for preview/injection.
	 * Always returns PrefilterRT with normalized data copied/prepared into it.
	 * Call this from the game thread before injection.
	 *
	 * @return Normalized texture suitable for display, or nullptr if not ready
	 */
	UTextureRenderTarget2D* PreparePreviewTexture();

	/**
	 * Get the pre-filter accumulated result (before median filter applied during finalization).
	 * Useful for re-filtering with different settings.
	 */
	UTextureRenderTarget2D* GetPrefilterResult() const
	{
		return PrefilterRT;
	}

	/**
	 * Get the sample render target (where individual samples are rendered).
	 */
	UTextureRenderTarget2D* GetSampleRT() const
	{
		return SampleRT;
	}

	/**
	 * Get ping-pong accumulation buffer A
	 */
	UTextureRenderTarget2D* GetAccumA() const
	{
		return AccumA;
	}

	/**
	 * Get ping-pong accumulation buffer B
	 */
	UTextureRenderTarget2D* GetAccumB() const
	{
		return AccumB;
	}

	/**
	 * Copy current normalized result to an external render target.
	 *
	 * @param OutputRT        Target render target
	 * @param bDrawProgress   Draw progress bar overlay (for preview)
	 */
	void CopyToOutput(UTextureRenderTarget2D* OutputRT, bool bDrawProgress = false);

	/**
	 * Copy current normalized result to an external render target with cropping.
	 * Used when overscan needs to be cropped from the output.
	 *
	 * @param OutputRT       Target render target (should already be sized to cropped dimensions)
	 * @param SourceUVMin    Minimum UV coordinates of source region to copy
	 * @param SourceUVMax    Maximum UV coordinates of source region to copy
	 * @param bDrawProgress  Draw progress bar overlay (for preview)
	 */
	void CopyToOutputCropped(
		UTextureRenderTarget2D* OutputRT,
		const FVector2f& SourceUVMin,
		const FVector2f& SourceUVMax,
		bool bDrawProgress = false
	);

	/**
	 * Render with post-processing injection.
	 * Injects the accumulated result at MotionBlur pass, then captures with the following post process applied.
	 * If Overscan > 0, post-processes at internal resolution then crops to output.
	 *
	 * @param OutputRT            - Target render target for post-processed result
	 * @param bAllowSceneFringe   - Whether to allow engine's SceneFringe (lateral CA)
	 * @param ProgressBarFraction - Progress bar value (0-1), negative to disable
	 * @param Overscan            - Overscan fraction (e.g. 0.1 for 10%), 0 for no cropping
	 *
	 * @return true if rendering succeeded
	 */
	bool RenderWithPostProcessing(
		UTextureRenderTarget2D* OutputRT,
		bool bAllowSceneFringe = false,
		float ProgressBarFraction = -1.0f,
		float Overscan = 0.0f
	);

	/**
	 * Get current progress information.
	 */
	const AccumulationDOF::FApertureSamplerProgress& GetProgress() const
	{
		return Progress;
	}

	/**
	 * Set delegate for progress updates.
	 */
	void SetOnProgress(AccumulationDOF::FOnApertureSamplerProgress InDelegate)
	{
		OnProgressDelegate = MoveTemp(InDelegate);
	}

	/**
	 * Set delegate for completion notification.
	 */
	void SetOnComplete(AccumulationDOF::FOnApertureSamplerComplete InDelegate)
	{
		OnCompleteDelegate = MoveTemp(InDelegate);
	}

	//
	// Debugging sample saving functions.
	//

	/**
	 * Save last rendered aperture sample to disk.
	 *
	 * @param SampleIndex - Index of sample to save (used for filename suffix)
	 * @param Directory   - Output directory (relative to project or absolute)
	 *
	 * @param Prefix       Filename prefix
	 */
	void SaveSampleImage(int32 SampleIndex, const FString& Directory, const FString& Prefix);

	/**
	 * Save final accumulated frame to disk.
	 *
	 * @param Directory - Output directory
	 * @param Prefix    - Filename prefix
	 */
	void SaveFinalImage(const FString& Directory, const FString& Prefix);

	/**
	 * Manually accumulate a sample that was rendered externally to SampleRT.
	 * Used by MRQ which renders via its own view family.
	 *
	 * @param SampleIndex      - Index of the sample
	 * @param ApertureOffsetCm - Aperture offset in cm
	 * @param SpectralWeight   - RGB weight
	 */
	void AccumulateExternalSample(
		int32 SampleIndex,
		const FVector2f& ApertureOffsetCm,
		const FVector3f& SpectralWeight = FVector3f(1.0f, 1.0f, 1.0f)
	);

	/**
	 * Normalize after all samples have been accumulated.
	 */
	void FinalizeAccumulation();

	/**
	 * Get aperture offset for a specific sample index, per the surrent sampling pattern.
	 */
	FVector2f GetApertureOffset(int32 SampleIndex) const;

	/**
	 * Get AA jitter offset for the given specific sample index.
	 * Returns jitter in [-0.5, 0.5] pixel units and using the r.AccumulationDOF.JitterAA.Sequence cvar sequence.
	 */
	FVector2f GetJitterOffset(int32 SampleIndex) const;

	/**
	 * Get AA jitter in clip-space format for HackAddTemporalAAProjectionJitter().
	 * This should be called after view creation and applied via View->ViewMatrices.HackAddTemporalAAProjectionJitter().
	 *
	 * @param SampleIndex - Index of the aperture sample
	 * @param Resolution  - Render target resolution
	 *
	 * @return Jitter offset in clip-space units (matching MRQ convention with negated Y)
	 */
	FVector2D GetJitterForProjectionMatrix(int32 SampleIndex, const FIntPoint& Resolution) const;

	/**
	 * Get array ref. of aperture offsets.
	 */
	const TArray<FVector2f>& GetApertureOffsetsRef() const
	{
		return ApertureOffsetsCm;
	}

	/**
	 * Compute off-axis projection matrix for a sample.
	 *
	 * @param SampleIndex           - Index of the aperture sample
	 * @param FocusDistanceOverride - Override focus distance (-1 to not override)
	 * @param InCameraView          - Camera view info; the near clip plane is read from here.
	 * @param OutMatrix             - Output projection matrix
	 *
	 * @return true if computation succeeded
	 */
	bool ComputeProjectionMatrix(
		int32 SampleIndex,
		float FocusDistanceOverride,
		const FMinimalViewInfo& InCameraView,
		FMatrix& OutMatrix
	) const;

	/**
	 * Compute sample parameters for UApertureSamplingRenderer.
	 * Centralizes the projection, focus, and lens aberration calculations.
	 *
	 * @param SampleIndex           - Index of the aperture sample
	 * @param FocusDistanceOverride - Override focus distance (-1 to not override)
	 *
	 * @return Sample parameters struct
	 */
	AccumulationDOF::FApertureSampleParams ComputeSampleParams(int32 SampleIndex, float FocusDistanceOverride = -1.0f) const;

	/**
	 * Compute spectral focus distance for axial chromatic aberration.
	 *
	 * @param NominalFocusCm             - Camera's nominal focus distance in cm
	 * @param NormalizedSpectralPosition - Position in [0,1]: [blue end, red end]
	 *
	 * @return Focus distance for this band in cm
	 */
	float ComputeSpectralFocusDistance(float NominalFocusCm, float NormalizedSpectralPosition) const;

	/**
	 * Computes spectral weight for axial chromatic aberration.
	 *
	 * @param NormalizedSpectralPosition - Position in [0,1]: [blue end, red end]
	 *
	 * @return RGB weight vector (R+G+B = 1)
	 */
	FVector3f ComputeSpectralWeight(float NormalizedSpectralPosition) const;

	/**
	 * @return The effective number of spectral bands for axial CA.
	 */
	int32 GetEffectiveBands() const;

	/**
	 * @return The total number of iterations (may be more than num samples if aberrations require it)
	 */
	int32 GetTotalIterations() const;

	/**
	 * Get the total spectral weight per channel for normalization.
	 */
	FVector3f GetTotalSpectralWeightPerChannel() const
	{
		return TotalSpectralWeightPerChannel;
	}

private:

	/** Setup render targets based on configuration. */
	void SetupRenderTargets();

	/** Generate aperture sample offsets based on pattern. */
	void GenerateApertureOffsets();

	/** Initialize the internal renderer. */
	void InitializeRenderer();

	/** Render a single sample using UApertureSamplingRenderer. */
	void RenderSample(int32 SampleIndex, const FVector2f& ApertureOffsetCm, float FocusDistanceOverride = -1.0f);

	/** Accumulate sample into accumulation buffer (internal path). */
	void AccumulateSampleInternal(
		int32 SampleIndex,
		const FVector2f& ApertureOffsetCm,
		const FVector3f& SpectralWeight
	);

	/** Normalize accumulated result. */
	void NormalizeAccumulation(float TotalWeightSum);

	/** Apply spectral lateral chromatic aberration. */
	void ApplySpectralLateralCA();

	/** Apply median filter. */
	void ApplyMedianFilter();

	/** Clear accumulation buffers. */
	void ClearAccumulationBuffers();

	/** Cache current state for change detection. */
	void CacheCurrentState();

	/** Check if camera changed significantly (to restart the capture if so). */
	bool HasSignificantChanges() const;

	/** Notify progress delegate. */
	void NotifyProgress();

	/** Pre-compute total spectral weights for normalization. */
	void PrecomputeSpectralWeights();

private:

	/** Determines the behavior of the sampler */
	AccumulationDOF::FApertureSamplerConfig Config;

	/** Information about the camera being rendered */
	AccumulationDOF::FApertureSamplerCameraState CameraState;

	/** How far along we are in sample render accumulation */
	AccumulationDOF::FApertureSamplerProgress Progress;

	/** Initialization flag */
	bool bIsInitialized = false;

	/** Used to know if re-filtering would be valid */
	bool bHasValidAccumulatedData = false;

	/** RT to hold aperture sample render */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> SampleRT;

	/** Ping pong RT A */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> AccumA;

	/** Ping pong RT B */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> AccumB;

	/** Caches the last accumulation to re-filter without re-rendering */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PrefilterRT;

	/** Intermediate RT for post-processing when overscan cropping is needed */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PostProcessRT;

	/** Array of aperture offsets to render from per the jitter pattern */
	TArray<FVector2f> ApertureOffsetsCm;

	/** Axial chromatic aberration weight per channel cache */
	FVector3f TotalSpectralWeightPerChannel = FVector3f(0.0f);

	/** Accumulated weight sum */
	float AccumulatedWeightSum = 0.0f;

	/** Last camera transform for change detection */
	FTransform LastCameraTransform;

	/** Last focus distance for change detection */
	float LastFocusDistance = 0.0f;

	/** Last aperture for change detection */
	float LastAperture = 0.0f;

	/** Last focal length for change detection */
	float LastFocalLength = 0.0f;

	/** Whether cached change detection values are valid */
	bool bLastChangeDetectionValid = false;

	/** Internal renderer. */
	UPROPERTY(Transient)
	TObjectPtr<UApertureSamplingRenderer> Renderer;

	/** BokehTexture reference (Config.BokehTexture points to same object) */
	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> BokehTextureRef;

	/** Progress update delegate */
	AccumulationDOF::FOnApertureSamplerProgress OnProgressDelegate;

	/** Frame completion delegate */
	AccumulationDOF::FOnApertureSamplerComplete OnCompleteDelegate;

	/** Saved value of r.DOF.Gather.ResolutionDivisor before we changed it */
	int32 SavedDOFGatherResDivisor = 2;

	/** Whether we modified the DOF gather resolution divisor CVar */
	bool bModifiedDOFGatherResDivisor = false;
};
