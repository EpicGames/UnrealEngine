// Copyright Epic Games, Inc. All Rights Reserved.

#include "ApertureSamplingRenderer.h"

#include "AccumulationDOFLog.h"
#include "AccumulationDOFSceneViewExtension.h"
#include "ApertureSampler.h"

#include "Camera/CameraComponent.h"
#include "CanvasTypes.h"
#include "CineCameraComponent.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "Interfaces/Interface_PostProcessVolume.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "LegacyScreenPercentageDriver.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "SceneViewExtensionContext.h"
#include "TextureResource.h"

using namespace AccumulationDOF;

UApertureSamplingRenderer::UApertureSamplingRenderer()
{
}

void UApertureSamplingRenderer::Initialize(UWorld* InWorld, FSceneViewStateInterface* InExposureViewState)
{
	if (World.Get() != InWorld)
	{
		SceneViewState.Destroy();
		bIsInitialized = false;
	}

	World = InWorld;
	ExposureViewState = InExposureViewState;
}

void UApertureSamplingRenderer::Shutdown()
{
	SceneViewState.Destroy();
	bIsInitialized = false;
}

void UApertureSamplingRenderer::SetupRenderTargets(const FApertureSamplingConfig& Config)
{
	CurrentConfig = Config;

	if (Config.Resolution.X < 1 || Config.Resolution.Y < 1)
	{
		UE_LOGF(LogAccumulationDOF, Error, "SetupRenderTargets: Invalid resolution %dx%d", Config.Resolution.X, Config.Resolution.Y);
		return;
	}

	// Create or resize internal render target (GC handled via UPROPERTY)
	if (!InternalRT || InternalRT->SizeX != Config.Resolution.X || InternalRT->SizeY != Config.Resolution.Y)
	{
		InternalRT = NewObject<UTextureRenderTarget2D>(this);
		InternalRT->RenderTargetFormat = RTF_RGBA16f;
		InternalRT->ClearColor = FLinearColor::Transparent;
		InternalRT->bAutoGenerateMips = false;
		InternalRT->InitAutoFormat(Config.Resolution.X, Config.Resolution.Y);
		InternalRT->UpdateResourceImmediate(true);
	}

	// Initialize scene view state for temporal history (TAA/TSR)
	if (!SceneViewState.GetReference() && World.IsValid())
	{
		SceneViewState.Allocate(World->GetFeatureLevel());
	}

	// Add Lumen scene data to the view state
	if (SceneViewState.GetReference() && World.IsValid() && World->Scene)
	{
		// This is a no-op if the data is already there
		SceneViewState.GetReference()->AddLumenSceneData(World->Scene, 1.0f);
	}

	bIsInitialized = true;
}

bool UApertureSamplingRenderer::IsInitialized() const
{
	return bIsInitialized && World.IsValid() && World->Scene && InternalRT && InternalRT->GetResource();
}

bool UApertureSamplingRenderer::RenderSample(
	const FApertureSampleParams& Params,
	UTextureRenderTarget2D* OutputRT,
	UCineCameraComponent* CineCamera,
	float* OutCapturedSceneFringeIntensity)
{
	if (!IsInitialized())
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderSample: Renderer not initialized");
		return false;
	}

	if (!OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderSample: OutputRT is null");
		return false;
	}

	// Get render target resource for internal RT
	FRenderTarget* RenderTargetResource = InternalRT->GameThread_GetRenderTargetResource();
	if (!RenderTargetResource)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderSample: Failed to get internal RT resource");
		return false;
	}

	// Create View Family
	TSharedRef<FSceneViewFamilyContext> ViewFamily = CreateViewFamily(RenderTargetResource, Params);

	// Create View Init Options with projection matrix
	FSceneViewInitOptions InitOptions = CreateViewInitOptions(Params, ViewFamily.ToSharedPtr().Get());

	// Create and configure the scene view

	FSceneView* View = CreateSceneView(InitOptions, ViewFamily.ToSharedPtr().Get(), Params, CineCamera);

	if (!View)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderSample: Failed to create scene view");
		return false;
	}

	// Configure post-process settings
	ConfigureViewPostProcess(View, Params, CineCamera);

	// Create SVE in capture mode to extract scene color to OutputRT
	FAccumulationDOFSVESettings SVESettings = FAccumulationDOFSVESettings::FromCineCameraComponent(CineCamera);

	TSharedRef<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe> CaptureExtension =
		MakeShared<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe>(
			OutputRT, EAccumulationDOFSVEMode::Capture, SVESettings
		);

	// Configure DOFSplats if enabled
	if (Params.DOFSplatsFStop > UE_KINDA_SMALL_NUMBER)
	{
		CaptureExtension->SetDOFSplatsSettings(
			Params.DOFSplatsFStop,
			Params.FocusDistanceCm,
			Params.SensorWidthMm,
			Params.SqueezeFactor,
			Params.bForceNeutralBokeh
		);
	}

	// Configure SceneFringe allowance
	CaptureExtension->SetAllowSceneFringe(Params.bAllowSceneFringe);

	// Add SVE to view family
	ViewFamily->ViewExtensions.Add(CaptureExtension);

	// Setup view for this extension
	CaptureExtension->SetupView(*ViewFamily, *View);

	// Create canvas and submit render
	FCanvas Canvas(RenderTargetResource, nullptr, World.Get(), World->GetFeatureLevel(),
		FCanvas::CDM_DeferDrawing, 1.0f);

	GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

	// Capture the SceneFringe intensity that was recorded by the SVE
	if (OutCapturedSceneFringeIntensity)
	{
		*OutCapturedSceneFringeIntensity = CaptureExtension->GetCapturedSceneFringeIntensity();
	}

	return true;
}

bool UApertureSamplingRenderer::RenderWithInjection(
	UTextureRenderTarget2D* AccumulatedRT,
	UTextureRenderTarget2D* OutputRT,
	UCineCameraComponent* CineCamera,
	ESceneCaptureSource CaptureSource,
	bool bAllowSceneFringe,
	bool bEnableMotionBlur,
	bool bWorldIsPaused,
	float ProgressBarFraction
)
{
	if (!IsInitialized())
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderWithInjection: Renderer not initialized");
		return false;
	}

	if (!AccumulatedRT || !OutputRT || !CineCamera)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderWithInjection: Invalid parameters (AccumulatedRT=%p, OutputRT=%p, CineCamera=%p)",
			AccumulatedRT, OutputRT, CineCamera);
		return false;
	}

	// Ensure output RT matches our internal resolution
	if (OutputRT->SizeX != CurrentConfig.Resolution.X || OutputRT->SizeY != CurrentConfig.Resolution.Y)
	{
		OutputRT->ResizeTarget(CurrentConfig.Resolution.X, CurrentConfig.Resolution.Y);
	}

	// Get render target resource for output
	FRenderTarget* OutputRTResource = OutputRT->GameThread_GetRenderTargetResource();
	if (!OutputRTResource)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApertureSamplingRenderer::RenderWithInjection: Failed to get output RT resource");
		return false;
	}

	// Create show flags for injection pass

	FEngineShowFlags ShowFlags(ESFIM_Game);
	ShowFlags.SetDepthOfField(false);
	ShowFlags.SetMotionBlur(bEnableMotionBlur);

	// Create view family for the injection pass
	TSharedRef<FSceneViewFamilyContext> ViewFamily = MakeShared<FSceneViewFamilyContext>(
		FSceneViewFamily::ConstructionValues(
			OutputRTResource,
			World->Scene,
			ShowFlags
		)
		.SetTime(FGameTime::CreateUndilated(World->GetTimeSeconds(), World->GetDeltaSeconds()))
		.SetRealtimeUpdate(true)
	);

	ViewFamily->SceneCaptureSource = CaptureSource;
	ViewFamily->bIsMainViewFamily = true;
	ViewFamily->bWorldIsPaused = bWorldIsPaused;

	// Gather active view extensions from engine
	ViewFamily->ViewExtensions.Append(
		GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(World->Scene))
	);

	// Setup extensions for this family
	for (FSceneViewExtensionRef& ViewExt : ViewFamily->ViewExtensions)
	{
		ViewExt->SetupViewFamily(*ViewFamily);
	}

	// Set up screen percentage interface
	if (ViewFamily->GetScreenPercentageInterface() == nullptr)
	{
		float ResolutionFraction = CurrentConfig.ScreenPercentageFraction;
		if (ResolutionFraction < 0.0f)
		{
			ResolutionFraction = FLegacyScreenPercentageDriver::GetCVarResolutionFraction();
		}
		ViewFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(*ViewFamily, ResolutionFraction));
		ViewFamily->EngineShowFlags.ScreenPercentage = (ResolutionFraction != 1.0f);
	}

	// Build view init options from camera
	const FIntPoint RenderResolution = OutputRTResource->GetSizeXY();
	if (RenderResolution.X < 1 || RenderResolution.Y < 1)
	{
		UE_LOGF(LogAccumulationDOF, Error, "RenderWithInjection: Invalid resolution %dx%d", RenderResolution.X, RenderResolution.Y);
		return false;
	}

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily.ToSharedPtr().Get();
	ViewInitOptions.ViewOrigin = CineCamera->GetComponentLocation();
	ViewInitOptions.ViewLocation = CineCamera->GetComponentLocation();
	ViewInitOptions.ViewRotation = CineCamera->GetComponentRotation();
	ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint(0, 0), RenderResolution));

	// Build view rotation matrix
	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(CineCamera->GetComponentRotation());
	ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	// Build projection matrix from camera settings
	FMinimalViewInfo CameraView;
	CineCamera->GetCameraView(0.0f, CameraView);

	// Use perspective projection
	const float HalfFOVRadians = FMath::DegreesToRadians(CameraView.FOV * 0.5f);
	const float AspectRatio = static_cast<float>(RenderResolution.X) / static_cast<float>(RenderResolution.Y);

	// Build standard perspective projection matrix (always reversed Z)
	ViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
		HalfFOVRadians,
		AspectRatio,
		1.0f,
		CameraView.GetFinalPerspectiveNearClipPlane()
	);

	// FSceneViewInitOptions::FOV is consumed directly by hair strand rasterization.
	ViewInitOptions.FOV = CameraView.FOV;
	ViewInitOptions.DesiredFOV = CameraView.FOV;

	// Assign scene view state for temporal history
	ViewInitOptions.SceneViewStateInterface = SceneViewState.GetReference();

	// Share exposure from main viewport if configured
	if (ExposureViewState)
	{
		ViewInitOptions.ExposureSceneViewStateInterface = ExposureViewState;
	}

	// Create the view
	FSceneView* View = new FSceneView(ViewInitOptions);
	ViewFamily->Views.Add(View);

	// Initialize post-process settings from PP volumes and camera
	View->StartFinalPostprocessSettings(ViewInitOptions.ViewLocation);
	View->OverridePostProcessSettings(CameraView.PostProcessSettings, CameraView.PostProcessBlendWeight);
	View->EndFinalPostprocessSettings(ViewInitOptions);

	// Configure view settings
	View->bCameraCut = false;
	View->bIsOfflineRender = true;

	// Injection SVE

	FAccumulationDOFSVESettings SVESettings = FAccumulationDOFSVESettings::FromCineCameraComponent(CineCamera);
	const EAccumulationDOFSVEMode InjectionMode = bEnableMotionBlur
		? EAccumulationDOFSVEMode::InjectViaTemporalUpscaler
		: EAccumulationDOFSVEMode::Inject;

	TSharedRef<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe> InjectionExtension =
		MakeShared<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe>(
			AccumulatedRT, InjectionMode, SVESettings
		);

	// Configure whether we use the default lateral CA or the cinematic version.
	InjectionExtension->SetAllowSceneFringe(bAllowSceneFringe);

	// Configure progress bar
	if (ProgressBarFraction >= 0.0f)
	{
		InjectionExtension->SetProgressBar(ProgressBarFraction);
	}

	// Add injection SVE to view family
	ViewFamily->ViewExtensions.Add(InjectionExtension);

	// Setup view extensions (including the injection extension we just added)
	for (int32 ViewExtIndex = 0; ViewExtIndex < ViewFamily->ViewExtensions.Num(); ViewExtIndex++)
	{
		ViewFamily->ViewExtensions[ViewExtIndex]->SetupView(*ViewFamily, *View);
	}

	// Create canvas and submit render
	FCanvas Canvas(OutputRTResource, nullptr, World.Get(), World->GetFeatureLevel(),
		FCanvas::CDM_DeferDrawing, 1.0f);

	GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.ToSharedPtr().Get());

	return true;
}

TSharedRef<FSceneViewFamilyContext> UApertureSamplingRenderer::CreateViewFamily(
	FRenderTarget* RenderTarget,
	const FApertureSampleParams& Params) const
{
	FEngineShowFlags ShowFlags = CurrentConfig.ShowFlags;

	// Configure ShowFlags based on parameters
	if (CurrentConfig.bDisableEngineDOF)
	{
		// Enable engine DOF only if DOFSplats is being used
		ShowFlags.SetDepthOfField(Params.DOFSplatsFStop > UE_KINDA_SMALL_NUMBER);
	}

	ShowFlags.SetMotionBlur(Params.bEnableMotionBlur);

	// TAA configuration based on AA method

	const bool bUseTemporalAA =
		   (Params.AntiAliasing == EAntiAliasingMethod::AAM_TemporalAA)
		|| (Params.AntiAliasing == EAntiAliasingMethod::AAM_TSR);

	ShowFlags.SetTemporalAA(bUseTemporalAA);
	ShowFlags.SetAntiAliasing(bUseTemporalAA);

	// Apply view mode ShowFlags
	switch (CurrentConfig.ViewModeIndex)
	{
	case VMI_Lit_DetailLighting:
		ShowFlags.SetOverrideDiffuseAndSpecular(true);
		break;
	case VMI_LightingOnly:
		ShowFlags.SetLightingOnlyOverride(true);
		break;
	case VMI_ReflectionOverride:
		ShowFlags.SetReflectionOverride(true);
		break;
	default:
		break;
	}

	TSharedRef<FSceneViewFamilyContext> ViewFamily = MakeShared<FSceneViewFamilyContext>(
		FSceneViewFamily::ConstructionValues(
			RenderTarget,
			World->Scene,
			ShowFlags
		)
		.SetTime(FGameTime::CreateUndilated(World->GetTimeSeconds(), World->GetDeltaSeconds()))
		.SetRealtimeUpdate(true)
	);

	// bWorldIsPaused controls temporal history accumulation.
 	ViewFamily->bWorldIsPaused = Params.bWorldIsPaused;

	// Configure final ouput capture stage
	ViewFamily->SceneCaptureSource = CurrentConfig.CaptureSource;

	// tbd...
	ViewFamily->bIsMainViewFamily = true;

	// Gather active view extensions from engine
	ViewFamily->ViewExtensions.Append(
		GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(World->Scene))
	);

	// Setup extensions for this family
	for (FSceneViewExtensionRef& ViewExt : ViewFamily->ViewExtensions)
	{
		ViewExt->SetupViewFamily(*ViewFamily);
	}

	// Set up screen percentage interface (required for rendering)
	if (ViewFamily->GetScreenPercentageInterface() == nullptr)
	{
		float ResolutionFraction = CurrentConfig.ScreenPercentageFraction;
		if (ResolutionFraction < 0.0f)
		{
			ResolutionFraction = FLegacyScreenPercentageDriver::GetCVarResolutionFraction();
		}
		ViewFamily->SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(*ViewFamily, ResolutionFraction));
		ViewFamily->EngineShowFlags.ScreenPercentage = (ResolutionFraction != 1.0f);
	}

	return ViewFamily;
}

FSceneViewInitOptions UApertureSamplingRenderer::CreateViewInitOptions(
	const FApertureSampleParams& Params,
	FSceneViewFamily* ViewFamily)
{
	check(ViewFamily);
	check(ViewFamily->RenderTarget);

	const FIntPoint RenderResolution = ViewFamily->RenderTarget->GetSizeXY();

	// Compute sample camera location (base location + aperture offset)
	const FQuat CameraQuat = Params.CameraRotation.Quaternion();
	const FVector RightVec = CameraQuat.GetRightVector();
	const FVector UpVec = CameraQuat.GetUpVector();

	const FVector SampleLocation = Params.CameraLocation +
		RightVec * Params.ApertureOffsetCm.X +
		UpVec * Params.ApertureOffsetCm.Y;

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.ViewOrigin = SampleLocation;
	ViewInitOptions.ViewLocation = SampleLocation;
	ViewInitOptions.ViewRotation = Params.CameraRotation;
	ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint(0, 0), RenderResolution));

	// Build view rotation matrix with UE's axis conversion (rotate 90 degrees)
	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(Params.CameraRotation);
	ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1)
	);

	// Use pre-computed off-axis projection matrix
	ViewInitOptions.ProjectionMatrix = Params.ProjectionMatrix;

	// Set FOV. Certain graphics features (e.g. groom) expect an explicit FOV and do not try to derived it from the projection matrix.
	ViewInitOptions.FOV = Params.FOVDegrees;
	ViewInitOptions.DesiredFOV = Params.FOVDegrees;

	// Assign scene view state for temporal history
	ViewInitOptions.SceneViewStateInterface = SceneViewState.GetReference();

	// Share exposure from main viewport if configured
	if (ExposureViewState)
	{
		ViewInitOptions.ExposureSceneViewStateInterface = ExposureViewState;
	}

	// Enable ray tracing for this view (needed for Lumen, RT reflections, etc.)
	// Note: bIsSceneCapture is NOT set - we want full rendering like the main view
	ViewInitOptions.bSceneCaptureUsesRayTracing = Params.bUseRayTracing;

	return ViewInitOptions;
}

FSceneView* UApertureSamplingRenderer::CreateSceneView(
	const FSceneViewInitOptions& InitOptions,
	FSceneViewFamily* ViewFamily,
	const FApertureSampleParams& Params,
	UCineCameraComponent* CineCamera)
{
	// Create our view based on the Init Options
	FSceneView* View = new FSceneView(InitOptions);
	ViewFamily->Views.Add(View);

	// Start post-process settings pipeline
	// This initializes FinalPostProcessSettings from CVars (including Lumen GI/Reflection methods)
	// and applies PP volume blending via World->AddPostProcessingSettings
	View->StartFinalPostprocessSettings(InitOptions.ViewLocation);

	// Apply camera post-process settings on top of the blended PP volume settings
	if (CineCamera)
	{
		FMinimalViewInfo ViewInfo;
		CineCamera->GetCameraView(0.0f, ViewInfo);
		View->OverridePostProcessSettings(ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight);
	}

	// Apply DOFSensorScale for tiling support
	View->FinalPostProcessSettings.DepthOfFieldSensorWidth *= Params.DOFSensorScale;

	View->EndFinalPostprocessSettings(InitOptions);

	// Apply AA jitter via HackAddTemporalAAProjectionJitter (must be after view creation).
	if (UApertureSampler* Sampler = GetTypedOuter<UApertureSampler>())
	{
		const FIntPoint Resolution = ViewFamily->RenderTarget->GetSizeXY();
		const FVector2D Jitter = Sampler->GetJitterForProjectionMatrix(Params.SampleIndex, Resolution);
		if (!Jitter.IsNearlyZero())
		{
			View->ViewMatrices.HackAddTemporalAAProjectionJitter(Jitter);
		}
	}

	// Setup view extensions
	for (int ViewExt = 0; ViewExt < ViewFamily->ViewExtensions.Num(); ViewExt++)
	{
		ViewFamily->ViewExtensions[ViewExt]->SetupView(*ViewFamily, *View);
	}

	// Apply view mode override parameters

	if (ViewFamily->EngineShowFlags.OverrideDiffuseAndSpecular) // Detail Lighting
	{
		View->DiffuseOverrideParameter = FVector4f(
			GEngine->LightingOnlyBrightness.R,
			GEngine->LightingOnlyBrightness.G,
			GEngine->LightingOnlyBrightness.B, 
			0.0f
		);

		View->SpecularOverrideParameter = FVector4f(0.1f, 0.1f, 0.1f, 0.0f);
	}
	else if (ViewFamily->EngineShowFlags.LightingOnlyOverride)
	{
		View->DiffuseOverrideParameter = FVector4f(
			GEngine->LightingOnlyBrightness.R,
			GEngine->LightingOnlyBrightness.G,
			GEngine->LightingOnlyBrightness.B, 
			0.0f
		);

		View->SpecularOverrideParameter = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	}
	else if (ViewFamily->EngineShowFlags.ReflectionOverride)
	{
		View->DiffuseOverrideParameter   = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		View->SpecularOverrideParameter  = FVector4f(1.0f, 1.0f, 1.0f, 0.0f);
		View->NormalOverrideParameter    = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
		View->RoughnessOverrideParameter = FVector2f(0.0f, 0.0f);
	}

	return View;
}

void UApertureSamplingRenderer::ConfigureViewPostProcess(
	FSceneView* View,
	const FApertureSampleParams& Params,
	UCineCameraComponent* CineCamera) const
{
	check(View);

	// Override frame index for deterministic TAA
	View->OverrideFrameIndexValue = Params.SampleIndex;
	View->bCameraCut = false;
	View->AntiAliasingMethod = Params.AntiAliasing;
	View->bIsOfflineRender = true;

	// Motion blur
	if (!Params.bEnableMotionBlur)
	{
		View->FinalPostProcessSettings.MotionBlurAmount = 0.0f;
		View->FinalPostProcessSettings.bOverride_MotionBlurAmount = true;
	}

	// DOF rendering is controlled via ShowFlags.SetDepthOfField() in CreateViewFamily().
	// Do NOT zero DepthOfFieldFstop - it's needed for physical camera exposure calculation.
}
