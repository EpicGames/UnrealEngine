// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "UObject/Object.h"

#include "ApertureSamplingRenderer.generated.h"

class UCineCameraComponent;
class UWorld;
class UTextureRenderTarget2D;
struct FAccumulationDOFSVESettings;

namespace AccumulationDOF
{

/**
 * Encapsulates all rendering configuration for one sample.
 */
struct FApertureSampleParams
{
	/** Base camera world location */
	FVector CameraLocation = FVector::ZeroVector;

	/** Base camera world rotation */
	FRotator CameraRotation = FRotator::ZeroRotator;

	/** Aperture sample offset from optical axis in cm.*/
	FVector2f ApertureOffsetCm = FVector2f::ZeroVector;

	/** Pre-computed off-axis projection matrix for this sample */
	FMatrix ProjectionMatrix = FMatrix::Identity;

	/** On-axis horizontal field of view in degrees.
	 *  Default mirrors FSceneViewInitOptions::FOV's own default (90). 
	 */
	float FOVDegrees = 90.0f;

	/** Focal length in mm */
	float FocalLengthMm = 35.0f;

	/** F-stop (aperture) */
	float FStop = 2.8f;

	/** Focus distance in cm */
	float FocusDistanceCm = 100.0f;

	/** Sensor width in mm */
	float SensorWidthMm = 36.0f;

	/** Sensor height in mm */
	float SensorHeightMm = 24.0f;

	/** Anamorphic squeeze factor */
	float SqueezeFactor = 1.0f;

	/** DOFSplats f-stop (0 = disabled) */
	float DOFSplatsFStop = 0.0f;

	/** Force neutral bokeh (no petzval, circular aperture, no cat's eye) */
	bool bForceNeutralBokeh = false;

	/** Whether world is paused (freezes temporal history) */
	bool bWorldIsPaused = true;

	/** Anti-aliasing method to use */
	EAntiAliasingMethod AntiAliasing = EAntiAliasingMethod::AAM_None;

	/** Whether to enable motion blur */
	bool bEnableMotionBlur = false;

	/** Whether to use ray tracing */
	bool bUseRayTracing = true;

	/** Sample index for debug/logging and temporal history. INDEX_NONE if invalid. */
	int32 SampleIndex = INDEX_NONE;

	/** DOF sensor scale for tiling support (1.0 = no tiling) */
	float DOFSensorScale = 1.0f;

	/** Whether to allow SceneFringe (lateral CA) during capture */
	bool bAllowSceneFringe = false;
};

/**
 * Configuration for the aperture sampling renderer.
 * Applies to all samples.
 */
struct FApertureSamplingConfig
{
	/** Output resolution */
	FIntPoint Resolution = FIntPoint(1920, 1080);

	/** Final capture source */
	ESceneCaptureSource CaptureSource = ESceneCaptureSource::SCS_FinalColorHDR;

	/** ShowFlags */
	FEngineShowFlags ShowFlags = FEngineShowFlags(ESFIM_Game);

	/** Should realtime DOF be disabled or not */
	bool bDisableEngineDOF = true;

	/** Screen percentage fraction (0.1 to 4.0). Negative = use CVar. */
	float ScreenPercentageFraction = -1.0f;

	/** View mode for rendering (Lit, Detail Lighting, etc.) */
	EViewModeIndex ViewModeIndex = VMI_Lit;
};

} // namespace AccumulationDOF

/**
 * Renderer for aperture-sampled depth of field.
 */
UCLASS()
class UApertureSamplingRenderer : public UObject
{
	GENERATED_BODY()

public:

	UApertureSamplingRenderer();

	/** Initialize with world reference. Must be called after construction. */
	void Initialize(UWorld* InWorld, FSceneViewStateInterface* InExposureViewState = nullptr);

	/** Cleanup resources. */
	void Shutdown();

	/**
	 * Initialize/update render targets based on configuration.
	 * Must be called before RenderSample if resolution changed.
	 */
	void SetupRenderTargets(const AccumulationDOF::FApertureSamplingConfig& Config);

	/**
	 * Render a single aperture sample.
	 *
	 * @param Params     - Sample parameters (location, projection, etc.)
	 * @param OutputRT   - Render target to write result to (via SVE)
	 * @param CineCamera - Optional camera component for PP settings (can be null)
	 * @param OutCapturedSceneFringeIntensity - Optional output for captured SceneFringe value
	 *
	 * @return true on success
	 */
	bool RenderSample(
		const AccumulationDOF::FApertureSampleParams& Params,
		UTextureRenderTarget2D* OutputRT,
		UCineCameraComponent* CineCamera = nullptr,
		float* OutCapturedSceneFringeIntensity = nullptr
	);

	/**
	 * Render scene with injection of accumulated texture into post-processing pipeline.
	 *
	 * @param AccumulatedRT         - The accumulated DOF result to inject
	 * @param OutputRT              - Render target to write the final post-processed result to
	 * @param CineCamera            - Camera component for PP settings and location
	 * @param CaptureSource         - What stage of post-processing to capture
	 * @param bAllowSceneFringe     - Whether to allow engine lateral chromatic aberration or not
	 * @param bEnableMotionBlur     - Whether to enable motion blur during injection or not
	 * @param bWorldIsPaused        - Whether temporal history updates are frozen or not
	 * @param ProgressBarFraction   - Optional progress bar value (0-1, -1 to disable)
	 *
	 * @return true on success
	 */
	bool RenderWithInjection(
		UTextureRenderTarget2D* AccumulatedRT,
		UTextureRenderTarget2D* OutputRT,
		UCineCameraComponent* CineCamera,
		ESceneCaptureSource CaptureSource,
		bool bAllowSceneFringe = false,
		bool bEnableMotionBlur = false,
		bool bWorldIsPaused = false,
		float ProgressBarFraction = -1.0f
	);

	/** Get the current configuration. */
	const AccumulationDOF::FApertureSamplingConfig& GetConfig() const
	{
		return CurrentConfig;
	}

	/** Check if renderer is properly initialized. */
	bool IsInitialized() const;

	/** Get the internal render target (for debugging). */
	UTextureRenderTarget2D* GetInternalRenderTarget() const
	{
		return InternalRT;
	}

private:

	/** Create FSceneViewFamily with our configuration. */
	TSharedRef<FSceneViewFamilyContext> CreateViewFamily(
		FRenderTarget* RenderTarget,
		const AccumulationDOF::FApertureSampleParams& Params
	) const;

	/** Create FSceneViewInitOptions from parameters. */
	FSceneViewInitOptions CreateViewInitOptions(
		const AccumulationDOF::FApertureSampleParams& Params,
		FSceneViewFamily* ViewFamily
	);

	/** Create and configure FSceneView. */
	FSceneView* CreateSceneView(
		const FSceneViewInitOptions& InitOptions,
		FSceneViewFamily* ViewFamily,
		const AccumulationDOF::FApertureSampleParams& Params,
		UCineCameraComponent* CineCamera
	);

	/** Apply post-process settings from camera and configure for capture. */
	void ConfigureViewPostProcess(
		FSceneView* View,
		const AccumulationDOF::FApertureSampleParams& Params,
		UCineCameraComponent* CineCamera
	) const;

private:

	/** World reference (weak to avoid preventing GC during level transitions) */
	TWeakObjectPtr<UWorld> World = nullptr;

	/** Current configuration */
	AccumulationDOF::FApertureSamplingConfig CurrentConfig;

	/** Scene view state shared between aperture samples */
	FSceneViewStateReference SceneViewState;

	/** External view state for shared exposure (if set, shares exposure history with another view) */
	FSceneViewStateInterface* ExposureViewState = nullptr;

	/** Internal render target for scene capture output */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> InternalRT = nullptr;

	/** Tracks if we've been initialized */
	bool bIsInitialized = false;
};
