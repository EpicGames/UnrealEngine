// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFShaders.h"

#include "AccumulationDOFLog.h"
#include "CommonRenderResources.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderTargetPool.h"
#include "RHIStaticStates.h"
#include "ShaderParameterUtils.h"
#include "SystemTextures.h"
#include "TextureResource.h"

// Implement shader types
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFAccumulateVS, "/Plugin/AccumulationDOF/Private/AccumulationDOFAccumulate.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFAccumulatePS, "/Plugin/AccumulationDOF/Private/AccumulationDOFAccumulate.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFNormalizeVS,  "/Plugin/AccumulationDOF/Private/AccumulationDOFNormalize.usf",  "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFNormalizePS,  "/Plugin/AccumulationDOF/Private/AccumulationDOFNormalize.usf",  "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFCopyPS,       "/Plugin/AccumulationDOF/Private/AccumulationDOFCopy.usf",       "MainPS", SF_Pixel);
namespace AccumulationDOF
{

void AccumulateSample(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* PrevAccumRT,
	FTextureRenderTargetResource* SampleRT,
	FTextureRenderTargetResource* OutputRT,
	FRHITexture* BokehTextureRHI,
	FRHISamplerState* BokehSamplerRHI,
	const FAccumulateSampleParams& Params
)
{
	check(IsInRenderingThread());

	if (!PrevAccumRT || !SampleRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "AccumulateSample: Null render target");
		return;
	}

	FRHITexture* PrevAccumTexture = PrevAccumRT->GetRenderTargetTexture();
	FRHITexture* SampleTexture = SampleRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!PrevAccumTexture || !SampleTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "AccumulateSample: Failed to get RHI textures");
		return;
	}

	if (PrevAccumTexture == OutputTexture || SampleTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "AccumulateSample: Input and output textures must not alias");
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	// Register external textures
	FRDGTextureRef PrevAccumRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(PrevAccumTexture, TEXT("PrevAccum")));
	FRDGTextureRef SampleRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SampleTexture, TEXT("Sample")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("AccumOutput")));

	// Get shaders
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFAccumulateVS> VertexShader(GlobalShaderMap);
	TShaderMapRef<FAccumulationDOFAccumulatePS> PixelShader(GlobalShaderMap);

	// Setup shader parameters

	FAccumulationDOFAccumulatePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFAccumulatePS::FParameters>();

	PassParameters->PrevAccumTexture               = PrevAccumRDG;
	PassParameters->PrevAccumSampler               = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->SampleTexture                  = SampleRDG;
	PassParameters->SampleSampler                  = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->BokehTexture                   = BokehTextureRHI ? BokehTextureRHI : GWhiteTexture->TextureRHI.GetReference();
	PassParameters->BokehSampler                   = BokehSamplerRHI ? BokehSamplerRHI : TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->ApertureOffsetCm               = Params.ApertureOffsetCm;
	PassParameters->ApertureRadiusCm               = Params.ApertureRadiusCm;
	PassParameters->BokehWeightChannel             = Params.BokehWeightChannel;
	PassParameters->BokehUseTint                   = Params.bUseTint ? 1 : 0;
	PassParameters->BokehTintStrength              = Params.TintStrength;
	PassParameters->BokehEnabled                   = Params.bBokehEnabled ? 1 : 0;
	PassParameters->Petzval                        = Params.Petzval;
	PassParameters->PetzvalFalloffPower            = Params.PetzvalFalloffPower;
	PassParameters->PetzvalExclusionBoxExtents     = Params.PetzvalExclusionBoxExtents;
	PassParameters->PetzvalExclusionBoxRadius      = Params.PetzvalExclusionBoxRadius;
	PassParameters->SqueezeFactor                  = Params.SqueezeFactor;
	PassParameters->SpectralWeight                 = Params.SpectralWeight;
	PassParameters->BokehEdgeSoftness              = Params.BokehEdgeSoftness;
	PassParameters->BarrelRadiusCm                 = Params.BarrelRadiusCm;
	PassParameters->BarrelLengthCm                 = Params.BarrelLengthCm;
	PassParameters->SensorHalfSizeCm               = Params.SensorHalfSizeCm;
	PassParameters->FocalLengthCm                  = Params.FocalLengthCm;
	PassParameters->FocusDistanceCm                = Params.FocusDistanceCm;
	PassParameters->DiaphragmBladeCount            = Params.DiaphragmBladeCount;
	PassParameters->DiaphragmRotationRad           = Params.DiaphragmRotationRad;
	PassParameters->BokehShape                     = Params.BokehShape;
	PassParameters->CocRadiusToCircumscribedRadius = Params.CocRadiusToCircumscribedRadius;
	PassParameters->CocRadiusToIncircleRadius      = Params.CocRadiusToIncircleRadius;
	PassParameters->DiaphragmBladeRadius           = Params.DiaphragmBladeRadius;
	PassParameters->DiaphragmBladeCenterOffset     = Params.DiaphragmBladeCenterOffset;
	PassParameters->AntiAliasingWeight             = Params.AntiAliasingWeight;
	PassParameters->SubpixelOffset                 = Params.SubpixelOffset;
	PassParameters->TexelSize                      = Params.TexelSize;
	PassParameters->ComaStrength                   = Params.ComaStrength;
	PassParameters->RenderTargets[0]               = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AccumulationDOFAccumulate"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, OutputSize](FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;

			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		}
	);

	GraphBuilder.Execute();
}

void NormalizeAccumulation(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* AccumRT,
	FTextureRenderTargetResource* OutputRT,
	float InvWeightSum
)
{
	check(IsInRenderingThread());

	if (!AccumRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "NormalizeAccumulation: Null render target");
		return;
	}

	FRHITexture* AccumTexture = AccumRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!AccumTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "NormalizeAccumulation: Failed to get RHI textures");
		return;
	}

	if (AccumTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "NormalizeAccumulation: Input and output textures must not alias");
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	// Register external textures
	FRDGTextureRef AccumRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(AccumTexture, TEXT("Accum")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("NormalizeOutput")));

	// Get shaders
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFNormalizeVS> VertexShader(GlobalShaderMap);
	TShaderMapRef<FAccumulationDOFNormalizePS> PixelShader(GlobalShaderMap);

	// Setup shader parameters
	FAccumulationDOFNormalizePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFNormalizePS::FParameters>();
	PassParameters->AccumTexture = AccumRDG;
	PassParameters->AccumSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->InvWeightSum = InvWeightSum;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AccumulationDOFNormalize"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, OutputSize](FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		}
	);

	GraphBuilder.Execute();
}

void CopyTexture(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT
)
{
	check(IsInRenderingThread());

	if (!SourceRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyTexture: Null render target");
		return;
	}

	FRHITexture* SourceTexture = SourceRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!SourceTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyTexture: Failed to get RHI textures");
		return;
	}

	if (SourceTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyTexture: Source and output textures must not alias");
		return;
	}

	// Use direct RHI copy if formats and sizes match (more efficient), otherwise use shader for format conversion
	const bool bFormatsMatch = SourceTexture->GetFormat() == OutputTexture->GetFormat();
	const bool bSizesMatch = SourceTexture->GetSizeX() == OutputTexture->GetSizeX()
						  && SourceTexture->GetSizeY() == OutputTexture->GetSizeY();

	if (bFormatsMatch && bSizesMatch)
	{
		FRHICopyTextureInfo CopyInfo;
		RHICmdList.CopyTexture(SourceTexture, OutputTexture, CopyInfo);
		return;
	}

	// Shader-based copy for format conversion
	FRDGBuilder GraphBuilder(RHICmdList);

	FRDGTextureRef SourceRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("CopySource")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("CopyOutput")));

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFCopyPS> PixelShader(GlobalShaderMap);

	FAccumulationDOFCopyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFCopyPS::FParameters>();
	PassParameters->SourceTexture = SourceRDG;
	PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->FrameCounter = GFrameCounter;
	PassParameters->DitherStrength = (OutputTexture->GetFormat() == PF_FloatRGBA) ? (1.0f / 2048.0f) : 0.0f;
	PassParameters->SourceUVMin = FVector2f(0.0f, 0.0f);
	PassParameters->SourceUVMax = FVector2f(1.0f, 1.0f);
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());
	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		GlobalShaderMap,
		RDG_EVENT_NAME("AccumulationDOFCopy"),
		PixelShader,
		PassParameters,
		FIntRect(0, 0, OutputSize.X, OutputSize.Y));

	GraphBuilder.Execute();
}

void CopyCropped(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT,
	const FVector2f& SourceUVMin,
	const FVector2f& SourceUVMax
)
{
	check(IsInRenderingThread());

	// Validate and normalize UV region
	const FVector2f ClampedMin(FMath::Clamp(SourceUVMin.X, 0.0f, 1.0f), FMath::Clamp(SourceUVMin.Y, 0.0f, 1.0f));
	const FVector2f ClampedMax(FMath::Clamp(SourceUVMax.X, 0.0f, 1.0f), FMath::Clamp(SourceUVMax.Y, 0.0f, 1.0f));
	const FVector2f FinalUVMin(FMath::Min(ClampedMin.X, ClampedMax.X), FMath::Min(ClampedMin.Y, ClampedMax.Y));
	const FVector2f FinalUVMax(FMath::Max(ClampedMin.X, ClampedMax.X), FMath::Max(ClampedMin.Y, ClampedMax.Y));

	if (!SourceRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyCropped: Null render target");
		return;
	}

	FRHITexture* SourceTexture = SourceRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!SourceTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyCropped: Failed to get RHI textures");
		return;
	}

	if (SourceTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyCropped: Source and output textures must not alias");
		return;
	}

	// Shader-based copy with UV remapping for cropping
	FRDGBuilder GraphBuilder(RHICmdList);

	FRDGTextureRef SourceRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("CropSource")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("CropOutput")));

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFCopyPS> PixelShader(GlobalShaderMap);

	FAccumulationDOFCopyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFCopyPS::FParameters>();
	PassParameters->SourceTexture = SourceRDG;
	PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->FrameCounter = GFrameCounter;
	PassParameters->DitherStrength = (OutputTexture->GetFormat() == PF_FloatRGBA) ? (1.0f / 2048.0f) : 0.0f;
	PassParameters->SourceUVMin = FinalUVMin;
	PassParameters->SourceUVMax = FinalUVMax;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());
	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		GlobalShaderMap,
		RDG_EVENT_NAME("AccumulationDOFCopyCropped"),
		PixelShader,
		PassParameters,
		FIntRect(0, 0, OutputSize.X, OutputSize.Y));

	GraphBuilder.Execute();
}

} // namespace AccumulationDOF

// Median filter shaders (intented to clean up bokeh rendered with equally spaced samples)
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFMedianVS, "/Plugin/AccumulationDOF/Private/AccumulationDOFMedian.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFMedianPS, "/Plugin/AccumulationDOF/Private/AccumulationDOFMedian.usf", "MainPS", SF_Pixel);

// Injection shaders (for post-processing after accumulation)
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFInjectVS, "/Plugin/AccumulationDOF/Private/AccumulationDOFInject.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFInjectPS, "/Plugin/AccumulationDOF/Private/AccumulationDOFInject.usf", "MainPS", SF_Pixel);

// Lateral chromatic aberration shaders
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFLateralCAVS, "/Plugin/AccumulationDOF/Private/AccumulationDOFLateralCA.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FAccumulationDOFLateralCAPS, "/Plugin/AccumulationDOF/Private/AccumulationDOFLateralCA.usf", "MainPS", SF_Pixel);

namespace AccumulationDOF
{

void ApplyMedianFilter(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* InputRT,
	FTextureRenderTargetResource* OutputRT
)
{
	check(IsInRenderingThread());

	if (!InputRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyMedianFilter: Null render target");
		return;
	}

	FRHITexture* InputTexture = InputRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!InputTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyMedianFilter: Failed to get RHI textures");
		return;
	}

	if (InputTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyMedianFilter: Input and output textures must not alias");
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	// Register external textures
	FRDGTextureRef InputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InputTexture, TEXT("MedianInput")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("MedianOutput")));

	// Get shaders
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFMedianVS> VertexShader(GlobalShaderMap);
	TShaderMapRef<FAccumulationDOFMedianPS> PixelShader(GlobalShaderMap);

	// Setup shader parameters
	FAccumulationDOFMedianPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFMedianPS::FParameters>();
	PassParameters->InputTexture = InputRDG;
	PassParameters->InputSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	const int32 InputSizeX = FMath::Max(1, static_cast<int32>(InputTexture->GetSizeX()));
	const int32 InputSizeY = FMath::Max(1, static_cast<int32>(InputTexture->GetSizeY()));
	PassParameters->InputTexelSize = FVector2f(1.0f / InputSizeX, 1.0f / InputSizeY);
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AccumulationDOFMedianFilter"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, OutputSize](FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;

			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		}
	);

	GraphBuilder.Execute();
}

void InjectToSceneColor(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef AccumulatedRDG,
	FRDGTextureRef OutputRDG,
	const FIntRect& ViewRect,
	float ProgressBarFraction,
	bool bDrawProgressBar,
	float OverscanFraction,
	bool bDrawPreviewLabel,
	bool bIsFrozen,
	bool bApplyAspectFit,
	const FLensDistortionLUT* LensDistortionLUT
)
{
	check(AccumulatedRDG);
	check(OutputRDG);

	// Get shaders
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFInjectVS> VertexShader(GlobalShaderMap);
	TShaderMapRef<FAccumulationDOFInjectPS> PixelShader(GlobalShaderMap);

	// Get texture sizes
	FIntVector AccumSize = AccumulatedRDG->Desc.GetSize();
	FIntVector OutputSize = OutputRDG->Desc.GetSize();

	// Setup shader parameters
	FAccumulationDOFInjectPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFInjectPS::FParameters>();
	PassParameters->AccumulatedTexture = AccumulatedRDG;
	PassParameters->AccumulatedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PassParameters->ViewportRect = FVector4f(
		static_cast<float>(ViewRect.Min.X),
		static_cast<float>(ViewRect.Min.Y),
		static_cast<float>(ViewRect.Max.X),
		static_cast<float>(ViewRect.Max.Y)
	);

	PassParameters->AccumulatedTextureSize = FVector2f(
		static_cast<float>(AccumSize.X),
		static_cast<float>(AccumSize.Y)
	);

	PassParameters->OutputTextureSize = FVector2f(
		static_cast<float>(OutputSize.X),
		static_cast<float>(OutputSize.Y)
	);

	// Progress bar parameters
	PassParameters->ProgressBarFraction = ProgressBarFraction;
	PassParameters->bDrawProgressBar = bDrawProgressBar ? 1u : 0u;
	PassParameters->ViewportSize = FVector2f(
		static_cast<float>(ViewRect.Width()),
		static_cast<float>(ViewRect.Height())
	);
	PassParameters->OverscanFraction = OverscanFraction;
	PassParameters->bDrawPreviewLabel = bDrawPreviewLabel ? 1u : 0u;
	PassParameters->bIsFrozen = bIsFrozen ? 1u : 0u;
	PassParameters->bApplyAspectFit = bApplyAspectFit ? 1u : 0u;

	// Lens distortion LUT. When non-null and IsEnabled(), the inject shader
	// applies the undistorting displacement so distortion shows through ADOF inject paths
	// that bypass the engine's normal LUT consumer (TSR).

	const bool bApplyLensDistortion = (LensDistortionLUT != nullptr) && LensDistortionLUT->IsEnabled();

	PassParameters->bApplyLensDistortion = bApplyLensDistortion ? 1u : 0u;
	PassParameters->UndistortingDisplacementTexture = bApplyLensDistortion
		? LensDistortionLUT->UndistortingDisplacementTexture
		: GSystemTextures.GetBlackDummy(GraphBuilder);

	PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// MiniFont texture for preview label text rendering
	PassParameters->MiniFontTexture = GSystemTextures.AsciiTexture ? GSystemTextures.AsciiTexture->GetRHI() : GSystemTextures.WhiteDummy->GetRHI();

	PassParameters->FrameCounter = GFrameCounter;
	PassParameters->DitherStrength = 1.0f / 2048.0f;

	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AccumulationDOFInject"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, ViewRect](FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(
				static_cast<float>(ViewRect.Min.X),
				static_cast<float>(ViewRect.Min.Y),
				0.0f,
				static_cast<float>(ViewRect.Max.X),
				static_cast<float>(ViewRect.Max.Y),
				1.0f
			);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;

			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		}
	);
}

void CopyWithProgressBar(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT,
	float Progress
)
{
	check(IsInRenderingThread());

	if (!SourceRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyWithProgressBar: Null render target");
		return;
	}

	FRHITexture* SourceTexture = SourceRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!SourceTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyWithProgressBar: Failed to get RHI textures");
		return;
	}

	if (SourceTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "CopyWithProgressBar: Input and output textures must not alias");
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	// Register external textures
	FRDGTextureRef SourceRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("ProgressBarSource")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("ProgressBarOutput")));

	// Get texture sizes
	FIntVector OutputSize = OutputRDG->Desc.GetSize();
	FIntRect ViewRect(0, 0, OutputSize.X, OutputSize.Y);

	// Use InjectToSceneColor with progress bar enabled
	InjectToSceneColor(GraphBuilder, SourceRDG, OutputRDG, ViewRect, Progress, true);

	GraphBuilder.Execute();
}

void ApplyLateralCA(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SampleRT,
	FTextureRenderTargetResource* OutputRT,
	float IntensityUV,
	float StartOffset,
	const FVector2f& Center
)
{
	check(IsInRenderingThread());

	if (!SampleRT || !OutputRT)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyLateralCA: Null render target");
		return;
	}

	FRHITexture* SampleTexture = SampleRT->GetRenderTargetTexture();
	FRHITexture* OutputTexture = OutputRT->GetRenderTargetTexture();

	if (!SampleTexture || !OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyLateralCA: Failed to get RHI textures");
		return;
	}

	if (SampleTexture == OutputTexture)
	{
		UE_LOGF(LogAccumulationDOF, Error, "ApplyLateralCA: Input and output textures must not alias");
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	// Register external textures
	FRDGTextureRef SampleRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SampleTexture, TEXT("LateralCASample")));
	FRDGTextureRef OutputRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputTexture, TEXT("LateralCAOutput")));

	// Get shaders
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FAccumulationDOFLateralCAVS> VertexShader(GlobalShaderMap);
	TShaderMapRef<FAccumulationDOFLateralCAPS> PixelShader(GlobalShaderMap);

	// Setup shader parameters for spectral CA
	FAccumulationDOFLateralCAPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAccumulationDOFLateralCAPS::FParameters>();
	PassParameters->SampleTexture = SampleRDG;
	PassParameters->SampleSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->IntensityUV = IntensityUV;
	PassParameters->StartOffset = StartOffset;
	PassParameters->Center = Center;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputRDG, ERenderTargetLoadAction::ENoAction);

	FIntPoint OutputSize(OutputTexture->GetSizeX(), OutputTexture->GetSizeY());

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AccumulationDOFSpectralLateralCA"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, OutputSize](FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;

			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			
			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		}
	);

	GraphBuilder.Execute();
}

}
