// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphUtils.h"

class FTextureRenderTargetResource;
struct FLensDistortionLUT;

/**
 * Vertex shader for aperture sample accumulation pass
 */
class FAccumulationDOFAccumulateVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFAccumulateVS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFAccumulateVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Pixel shader for aperture sample accumulation pass
 */
class FAccumulationDOFAccumulatePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFAccumulatePS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFAccumulatePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevAccumTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, PrevAccumSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SampleTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SampleSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, BokehTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BokehSampler)
		SHADER_PARAMETER(FVector2f, ApertureOffsetCm)
		SHADER_PARAMETER(float, ApertureRadiusCm)
		SHADER_PARAMETER(int32, BokehWeightChannel)
		SHADER_PARAMETER(int32, BokehUseTint)
		SHADER_PARAMETER(float, BokehTintStrength)
		SHADER_PARAMETER(int32, BokehEnabled)

		// Anamorphic squeeze factor for bokeh
		SHADER_PARAMETER(float, SqueezeFactor)

		// Spectral weight for axial chromatic aberration (1,1,1 = no CA)
		SHADER_PARAMETER(FVector3f, SpectralWeight)

		// Soft falloff at aperture edge (0 = hard, 1 = very soft)
		SHADER_PARAMETER(float, BokehEdgeSoftness)

		// Petzval field curvature parameters (from PostProcessSettings)
		SHADER_PARAMETER(float, Petzval)
		SHADER_PARAMETER(float, PetzvalFalloffPower)
		SHADER_PARAMETER(FVector2f, PetzvalExclusionBoxExtents)
		SHADER_PARAMETER(float, PetzvalExclusionBoxRadius)

		// Barrel occlusion parameters (cat's eye bokeh)
		SHADER_PARAMETER(float, BarrelRadiusCm)
		SHADER_PARAMETER(float, BarrelLengthCm)
		SHADER_PARAMETER(FVector2f, SensorHalfSizeCm) // Half sensor size (cm) for ray direction
		SHADER_PARAMETER(float, FocalLengthCm)      // Focal length (cm) for ray direction
		SHADER_PARAMETER(float, FocusDistanceCm)    // Focus distance (cm) for focus plane calculation

		// Diaphragm blade parameters (from PostProcessSettings)
		SHADER_PARAMETER(int32, DiaphragmBladeCount)
		SHADER_PARAMETER(float, DiaphragmRotationRad)
		SHADER_PARAMETER(int32, BokehShape)                     // 0:Circle, 1:StraightBlades, 2:RoundedBlades
		SHADER_PARAMETER(float, CocRadiusToCircumscribedRadius) // Scale factor for circumscribed radius
		SHADER_PARAMETER(float, CocRadiusToIncircleRadius)      // Scale factor for incircle radius
		SHADER_PARAMETER(float, DiaphragmBladeRadius)           // Blade arc radius (for rounded blades)
		SHADER_PARAMETER(float, DiaphragmBladeCenterOffset)     // Blade arc center offset (for rounded blades)

		// AA reconstruction weight
		SHADER_PARAMETER(float, AntiAliasingWeight)

		// Bilinear sub-pixel splatting parameters
		SHADER_PARAMETER(FVector2f, SubpixelOffset)
		SHADER_PARAMETER(FVector2f, TexelSize)

		// Coma aberration strength (Seidel W131), normalized.
		SHADER_PARAMETER(float, ComaStrength)

		RENDER_TARGET_BINDING_SLOTS()

	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Vertex shader for normalization pass
 */
class FAccumulationDOFNormalizeVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFNormalizeVS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFNormalizeVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Pixel shader for normalizing accumulated weighted samples
 */
class FAccumulationDOFNormalizePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFNormalizePS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFNormalizePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AccumTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, AccumSampler)
		SHADER_PARAMETER(float, InvWeightSum)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

namespace AccumulationDOF
{

/**
 * Parameters for accumulating aperture samples with bokeh weighting.
 * Consolidates all shader parameters for AccumulateSample.
 */
struct FAccumulateSampleParams
{
	/** Aperture sample offset from optical axis in cm */
	FVector2f ApertureOffsetCm = FVector2f::ZeroVector;

	/** Aperture radius in cm */
	float ApertureRadiusCm = 0.0f;

	/** Bokeh weight channel (0 = Alpha, 1 = Luminance) */
	int32 BokehWeightChannel = 0;

	/** Whether to use bokeh tint */
	bool bUseTint = false;

	/** Bokeh tint strength (0-1) */
	float TintStrength = 0.0f;

	/** Whether bokeh texture is enabled */
	bool bBokehEnabled = false;

	/** Petzval field curvature coefficient */
	float Petzval = 0.0f;

	/** Petzval falloff power */
	float PetzvalFalloffPower = 1.0f;

	/** Petzval exclusion box extents */
	FVector2f PetzvalExclusionBoxExtents = FVector2f::ZeroVector;

	/** Petzval exclusion box corner radius */
	float PetzvalExclusionBoxRadius = 0.0f;

	/** Anamorphic squeeze factor */
	float SqueezeFactor = 1.0f;

	/** Bokeh edge softness (0 = hard, 1 = very soft) */
	float BokehEdgeSoftness = 0.0f;

	/** Barrel radius in cm for cat's eye vignetting */
	float BarrelRadiusCm = 0.0f;

	/** Barrel length in cm for cat's eye vignetting */
	float BarrelLengthCm = 0.0f;

	/** Half sensor size in cm for ray direction calculation */
	FVector2f SensorHalfSizeCm = FVector2f::ZeroVector;

	/** Focal length in cm for ray direction calculation */
	float FocalLengthCm = 0.0f;

	/** Focus distance in cm for focus plane calculation */
	float FocusDistanceCm = 0.0f;

	/** Number of diaphragm blades (0 = circular) */
	int32 DiaphragmBladeCount = 0;

	/** Diaphragm rotation in radians */
	float DiaphragmRotationRad = 0.0f;

	/** Bokeh shape: 0=Circle, 1=StraightBlades, 2=RoundedBlades */
	int32 BokehShape = 0;

	/** Scale factor for circumscribed radius */
	float CocRadiusToCircumscribedRadius = 1.0f;

	/** Scale factor for incircle radius */
	float CocRadiusToIncircleRadius = 1.0f;

	/** Blade arc radius for rounded blades */
	float DiaphragmBladeRadius = 0.0f;

	/** Blade arc center offset for rounded blades */
	float DiaphragmBladeCenterOffset = 0.0f;

	/** Spectral weight for axial chromatic aberration (1,1,1 = no CA) */
	FVector3f SpectralWeight = FVector3f(1.0f, 1.0f, 1.0f);

	/** AA reconstruction weight */
	float AntiAliasingWeight = 1.0f;

	/** Sub-pixel offset for bilinear splatting [0,1] */
	FVector2f SubpixelOffset = FVector2f(0.5f, 0.5f);

	/** Texel size (1/width, 1/height) for UV offset calculation */
	FVector2f TexelSize = FVector2f(0.0f, 0.0f);

	/** Coma aberration strength (Seidel W131), normalized */
	float ComaStrength = 0.0f;
};

/**
 * High-level accumulation function
 * Accumulates a sample into the accumulation buffer using custom shader with bokeh weighting
 */
void AccumulateSample(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* PrevAccumRT,
	FTextureRenderTargetResource* SampleRT,
	FTextureRenderTargetResource* OutputRT,
	FRHITexture* BokehTextureRHI,
	FRHISamplerState* BokehSamplerRHI,
	const FAccumulateSampleParams& Params
);

/**
 * High-level normalization function
 * Normalizes accumulated samples by total weight sum
 */
void NormalizeAccumulation(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* AccumRT,
	FTextureRenderTargetResource* OutputRT,
	float InvWeightSum
);

/**
 * Copies source texture to output texture.
 */
void CopyTexture(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT
);

/**
 * Copies a cropped region from source texture to output texture.
 * Used for overscan cropping where source is larger than destination.
 *
 * @param RHICmdList  - RHI command list
 * @param SourceRT    - Source render target (overscanned)
 * @param OutputRT    - Output render target (nominal size)
 * @param SourceUVMin - Minimum UV coordinates of source region to copy
 * @param SourceUVMax - Maximum UV coordinates of source region to copy
 */
void CopyCropped(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT,
	const FVector2f& SourceUVMin,
	const FVector2f& SourceUVMax
);

/**
 * Applies 3x3 median filter for noise reduction (per Life of a Bokeh - Siggraph 2018)
 */
void ApplyMedianFilter(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* InputRT,
	FTextureRenderTargetResource* OutputRT
);

/**
 * Inject accumulated DOF texture into scene color using RDG
 * Used by the scene view extension to replace scene color with accumulated result
 *
 * @param GraphBuilder        - The RDG builder
 * @param AccumulatedRDG      - The accumulated texture
 * @param OutputRDG           - The output scene color texture
 * @param ViewRect            - The viewport rectangle
 * @param ProgressBarFraction - Progress bar fill fraction (0.0 to 1.0), or negative to disable
 * @param bDrawProgressBar    - Whether to draw the progress bar overlay or not
 * @param OverscanFraction    - Overscan fraction for progress bar positioning
 * @param bDrawPreviewLabel   - Whether to draw "Accumulation DOF Preview" label (level editor only)
 * @param bIsFrozen           - Whether preview is frozen (appends "(FROZEN)" to label)
 * @param bApplyAspectFit     - When true, letterbox/pillarbox if the accumulator and viewport
 *                              aspects differ. Set false when the caller has already shrunk
 *                              the ViewRect to the target aspect.
 * @param LensDistortionLUT   - When non-null and IsEnabled(), the shader applies the undistorting
 *                              displacement to warp the accumulated buffer through the lens.
 */
ACCUMULATIONDOF_API void InjectToSceneColor(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef AccumulatedRDG,
	FRDGTextureRef OutputRDG,
	const FIntRect& ViewRect,
	float ProgressBarFraction = -1.0f,
	bool bDrawProgressBar = false,
	float OverscanFraction = 0.0f,
	bool bDrawPreviewLabel = false,
	bool bIsFrozen = false,
	bool bApplyAspectFit = true,
	const FLensDistortionLUT* LensDistortionLUT = nullptr
);

/**
 * Copies source texture to output texture with optional progress bar overlay.
 * Used for preview mode without post-processing.
 *
 * @param RHICmdList - RHI command list
 * @param SourceRT   - Source render target
 * @param OutputRT   - Output render target
 * @param Progress   - Progress bar fill fraction [0, 1]
 */
void CopyWithProgressBar(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SourceRT,
	FTextureRenderTargetResource* OutputRT,
	float Progress
);

/**
 * Apply spectral lateral chromatic aberration.
 * Uses multi-band spectral sampling for smooth rainbow fringes instead of simple RGB split.
 *
 * @param RHICmdList  - RHI command list
 * @param SampleRT    - Input sample render target
 * @param OutputRT    - Output render target (receives result with spectral chromatic aberration)
 * @param IntensityUV - Intensity in UV space (SceneFringeIntensity / min dimension)
 * @param StartOffset - Per-axis threshold where effect begins [0,1]
 * @param Center      - Center point in UV space
 */
void ApplyLateralCA(
	FRHICommandListImmediate& RHICmdList,
	FTextureRenderTargetResource* SampleRT,
	FTextureRenderTargetResource* OutputRT,
	float IntensityUV,
	float StartOffset,
	const FVector2f& Center
);

}

/**
 * Pixel shader for copying with format conversion
 */
class FAccumulationDOFCopyPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFCopyPS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFCopyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		// Dithering parameters for f32->f16 format conversion
		SHADER_PARAMETER(uint32, FrameCounter)
		SHADER_PARAMETER(float, DitherStrength)
		// Source UV crop region (for overscan cropping)
		SHADER_PARAMETER(FVector2f, SourceUVMin)
		SHADER_PARAMETER(FVector2f, SourceUVMax)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Vertex shader for median filter pass
 */
class FAccumulationDOFMedianVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFMedianVS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFMedianVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Pixel shader for 3x3 median filter (noise reduction)
 */
class FAccumulationDOFMedianPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFMedianPS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFMedianPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER(FVector2f, InputTexelSize)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

//
// Scene Color Injection Shaders (injection of accumulated texture so that it gets tonemapped)
//

/**
 * Vertex shader for scene color injection pass
 */
class FAccumulationDOFInjectVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFInjectVS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFInjectVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Pixel shader for injecting accumulated DOF result into scene color
 */
class FAccumulationDOFInjectPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFInjectPS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFInjectPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AccumulatedTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, AccumulatedSampler)
		SHADER_PARAMETER(FVector4f, ViewportRect)
		SHADER_PARAMETER(FVector2f, AccumulatedTextureSize)
		SHADER_PARAMETER(FVector2f, OutputTextureSize)
		SHADER_PARAMETER(float, ProgressBarFraction)
		SHADER_PARAMETER(uint32, bDrawProgressBar)
		SHADER_PARAMETER(FVector2f, ViewportSize)
		SHADER_PARAMETER(float, OverscanFraction)
		SHADER_PARAMETER(uint32, bDrawPreviewLabel)
		SHADER_PARAMETER(uint32, bIsFrozen)
		SHADER_PARAMETER(uint32, bApplyAspectFit)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER(uint32, FrameCounter)
		SHADER_PARAMETER(float, DitherStrength)
		// Lens distortion (optional). Active when bApplyLensDistortion != 0.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)
		SHADER_PARAMETER(uint32, bApplyLensDistortion)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

//
// Spectral Lateral Chromatic Aberration UV Shift Shaders
//

/**
 * Vertex shader for lateral CA UV shift pass
 */
class FAccumulationDOFLateralCAVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFLateralCAVS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFLateralCAVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

/**
 * Pixel shader for applying spectral lateral chromatic aberration.
 * Uses multi-band spectral sampling for smooth rainbow fringes.
 */
class FAccumulationDOFLateralCAPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulationDOFLateralCAPS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulationDOFLateralCAPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SampleTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SampleSampler)
		SHADER_PARAMETER(float, IntensityUV)   // Intensity in UV space
		SHADER_PARAMETER(float, StartOffset)   // Per-axis threshold where effect begins
		SHADER_PARAMETER(FVector2f, Center)    // Center in UV space
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};
