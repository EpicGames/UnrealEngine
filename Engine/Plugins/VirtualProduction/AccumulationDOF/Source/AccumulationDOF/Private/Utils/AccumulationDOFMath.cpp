// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFMath.h"

#include "Algo/Rotate.h"
#include "Math/Sobol.h"

namespace AccumulationDOFMath
{
	float Halton(int32 Index, int32 Base)
	{
		// Index is 0-based; internally we use (Index+1) to compute the radical inverse.
		// This avoids the degenerate case where Index=0 maps to 0.0.
		// E.g. Index=5 -> (5+1)=6 in base 2 is 110, radical inverse is 0.011 = 0.375

		if (Base < 2)
		{
			return 0.0f;
		}

		float Result = 0.0f;
		float PositionalWeight = 1.0f;
		int32 RemainingIndex = Index + 1;

		while (RemainingIndex > 0)
		{
			PositionalWeight = PositionalWeight / Base;
			Result = Result + PositionalWeight * (RemainingIndex % Base);
			RemainingIndex = RemainingIndex / Base;
		}

		return Result;
	}

	FVector2f ConcentricDiskSample(float U, float V)
	{
		// This implements the Shirley and Chiu's concentric disk mapping, which maps a point in a square
		// onto a disk, but keeping uniform distribution / density.
		// Intuition: Maps concentric squares to concentric rings.

		// Map [0,1) to [-1,1)
		const float A = 2.0f * U - 1.0f;
		const float B = 2.0f * V - 1.0f;

		if (A == 0.0f && B == 0.0f)
		{
			return FVector2f(0.0f, 0.0f);
		}

		float Theta;
		float R;

		// The dominating coordinate value will be the ring's radius R, since that is half the length of the square we're mapping to a ring.
		// Then linearly map the other one to an angle (angle/45 = proportion of the short side to R).
		// You can imagine a perfect grid that has equidistant samples in cardinal directions, and we are mapping those to angles,
		// which are really arc-distances, ensuring the samples are radially equidistant across all rings.

		if (A * A > B * B)
		{
			R = A;
			Theta = (PI / 4.0f) * (B / A);
		}
		else
		{
			R = B;
			Theta = (PI / 2.0f) - (PI / 4.0f) * (A / B);
		}

		return FVector2f(
			R * FMath::Cos(Theta),
			R * FMath::Sin(Theta)
		);
	}

	void GenerateApertureSamplesHalton(int32 NumSamples, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets)
	{
		if (NumSamples <= 0)
		{
			OutOffsets.Reset();
			return;
		}

		OutOffsets.Reset(NumSamples);

		const float EffectiveRadiusCm = ApertureRadiusCm * (1.0f - ApertureSampleMargin);

		for (int32 SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
		{
			// Use Halton sequence with bases 2 and 3 for low discrepancy sampling.
			// 2d Halton requires the bases of X and Y to be coprime (like 2 and 3). If they are not, then
			// the generated digits that express U and V will be related in such a way that all possible digit pairs are not traversed,
			// which would leave 2d regions unexplored (digits become congruent modulo the common prime factor).

			float U = Halton(SampleIdx, 2);
			float V = Halton(SampleIdx, 3);

			// Map to unit disk using Shirley-Chiu concentric mapping
			FVector2f UnitSample = ConcentricDiskSample(U, V);

			// Scale by effective aperture radius (slightly inset to avoid boundary issues)
			OutOffsets.Add(UnitSample * EffectiveRadiusCm);
		}
	}

	int32 ComputeRingCountForSamples(int32 DesiredSamples)
	{
		// Total samples for R rings (with center): 
		//     Samples = 1 + K*R*(R+1)/2 
		// Where:
		//     K is how many samples to add to each consecutively outer ring.
		// 
		// The equation comes from adding all the sample rings (including the center) knowing that each
		// one adds 6 additional samples to account for the increased circumference and keep the same radial distances on all rings.
		// 
		// So we need to solve for R (the number of rings).
		// 
		//     R = (-1 + sqrt(1 + 8*(Samples-1)/K)) / 2
		//
		// K should ideally be 6.28 to keep radial and tangential distances constant, since we can't have fractionals, 
		// we settle for 6 (see RingSamplesIncrement).

		if (DesiredSamples <= 1)
		{
			return 0;
		}

		const float K = static_cast<float>(RingSamplesIncrement);
		const float Discriminant = 1.0f + 8.0f * (DesiredSamples - 1) / K;
		const float R = (-1.0f + FMath::Sqrt(Discriminant)) / 2.0f;

		return FMath::CeilToInt32(R);
	}

	void GenerateApertureSamplesRing(int32 NumRings, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets)
	{
		const int32 TotalSamples = GetRingSampleCount(NumRings);
		OutOffsets.Reset(TotalSamples);

		// Center sample at index 0
		OutOffsets.Add(FVector2f(0.0f, 0.0f));

		if (NumRings <= 0)
		{
			return;
		}

		const float EffectiveRadiusCm = ApertureRadiusCm * (1.0f - ApertureSampleMargin);

		// Generate samples for each ring
		for (int32 RingId = 0; RingId < NumRings; ++RingId)
		{
			// Number of samples in this ring: (RingId+1) * RingSamplesIncrement
			const int32 RingSampleCount = (RingId + 1) * RingSamplesIncrement;

			// Radius of this ring (normalized 0-1, then scaled)
			// Rings are evenly spaced from center to edge
			const float NormalizedRadius = static_cast<float>(RingId + 1) / static_cast<float>(NumRings);
			const float RingRadius = NormalizedRadius * EffectiveRadiusCm;

			// Golden angle rotation per ring breaks radial alignment.
			// Each ring incrementally rotates by an irrational number of degree, ensuring
			// samples never align across rings regardless of ring count.
			// Without this rotation, certain radial streaks could appear.

			const float GoldenRotation = GoldenAngle * static_cast<float>(RingId);

			for (int32 SampleId = 0; SampleId < RingSampleCount; ++SampleId)
			{
				// Angular position is evenly spaced within ring, plus golden rotation

				const float Angle = TWO_PI * static_cast<float>(SampleId) / static_cast<float>(RingSampleCount) + GoldenRotation;

				OutOffsets.Add(FVector2f(
					RingRadius * FMath::Cos(Angle),
					RingRadius * FMath::Sin(Angle)
				));
			}
		}

		// Reorder each ring so that consecutive rings have minimal positional jump.
		// For each ring, find the sample closest to the previous ring's last sample,
		// then rotate the ring so that sample becomes first.

		int32 CurrentRingStart = 1; // Ring 0 starts after center sample

		for (int32 RingId = 0; RingId < NumRings; ++RingId)
		{
			const int32 RingSampleCount = (RingId + 1) * RingSamplesIncrement;
			const int32 CurrentRingEnd = CurrentRingStart + RingSampleCount;

			// Reference point is the last sample before this ring
			const FVector2f RefPoint = OutOffsets[CurrentRingStart - 1];

			// Find closest sample in current ring to RefPoint
			int32 ClosestIdx = CurrentRingStart;
			float MinDistSq = FLT_MAX;

			for (int32 SampleIdx = CurrentRingStart; SampleIdx < CurrentRingEnd; ++SampleIdx)
			{
				const float DistSq = FVector2f::DistSquared(OutOffsets[SampleIdx], RefPoint);
				if (DistSq < MinDistSq)
				{
					MinDistSq = DistSq;
					ClosestIdx = SampleIdx;
				}
			}

			// Rotate ring so ClosestIdx becomes the first sample of this ring
			if (ClosestIdx != CurrentRingStart)
			{
				// Rotate the subrange [CurrentRingStart, CurrentRingEnd)
				// Move (ClosestIdx - CurrentRingStart) elements from front to back
				const int32 RotateCount = ClosestIdx - CurrentRingStart;
				AlgoImpl::RotateInternal(
					OutOffsets.GetData() + CurrentRingStart,
					RingSampleCount,
					RotateCount
				);
			}

			CurrentRingStart = CurrentRingEnd;
		}
	}

	void GenerateApertureSamplesVogel(int32 NumSamples, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets)
	{
		if (NumSamples <= 0)
		{
			OutOffsets.Reset();
			return;
		}

		OutOffsets.Reset(NumSamples);

		// First sample at center (r=0)
		OutOffsets.Add(FVector2f(0.0f, 0.0f));

		if (NumSamples == 1)
		{
			return;
		}

		const float EffectiveRadiusCm = ApertureRadiusCm * (1.0f - ApertureSampleMargin);
		const float InvMaxIndex = 1.0f / static_cast<float>(NumSamples - 1);

		for (int32 SampleIdx = 1; SampleIdx < NumSamples; ++SampleIdx)
		{
			const float Theta = GoldenAngle * static_cast<float>(SampleIdx);
			const float NormalizedRadius = FMath::Sqrt(static_cast<float>(SampleIdx) * InvMaxIndex);
			const float Radius = NormalizedRadius * EffectiveRadiusCm;

			OutOffsets.Add(FVector2f(
				Radius * FMath::Cos(Theta),
				Radius * FMath::Sin(Theta)
			));
		}
	}

	void ComputeSensorCorners(float SensorWidthCm, float SensorHeightCm, float FocalLengthCm, FVector OutCorners[4])
	{
		const float HalfWidth = SensorWidthCm * 0.5f;
		const float HalfHeight = SensorHeightCm * 0.5f;

		// Lens is at 0, sensor is behind the lens at -FocalLengthCm

		OutCorners[0] = FVector(-HalfWidth, -HalfHeight, -FocalLengthCm); // Bottom-left
		OutCorners[1] = FVector( HalfWidth, -HalfHeight, -FocalLengthCm); // Bottom-right
		OutCorners[2] = FVector(-HalfWidth,  HalfHeight, -FocalLengthCm); // Top-left
		OutCorners[3] = FVector( HalfWidth,  HalfHeight, -FocalLengthCm); // Top-right
	}

	void ComputeFocusPlaneCorners(const FVector SensorCorners[4], float FocusDistanceCm, FVector OutFocusCorners[4])
	{
		// Rays from sensor through lens center project to focus plane.
		// Sensor is at negative focal length, lens at 0, focus plane at positive focus distance

		for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
		{
			// Scale factor is how many times the focal length fits in focus distance
			const float AbsZ = FMath::Abs(SensorCorners[CornerIdx].Z);

			if (AbsZ < KINDA_SMALL_NUMBER)
			{
				// Degenerate case - set to origin
				OutFocusCorners[CornerIdx] = FVector(0, 0, FocusDistanceCm);
				continue;
			}

			float Scale = FocusDistanceCm / AbsZ;

			OutFocusCorners[CornerIdx] = FVector(
				SensorCorners[CornerIdx].X * Scale,
				SensorCorners[CornerIdx].Y * Scale,
				FocusDistanceCm
			);
		}
	}

	void ComputeOffAxisFrustum(
		const FVector FocusCorners[4],
		const FVector2f& ApertureOffsetCm,
		float NearCm,
		float& OutLeft,
		float& OutRight,
		float& OutBottom,
		float& OutTop
	)
	{
		// Aperture offset in view space (on lens plane, Z=0)
		const FVector ApertureSamplePoint(ApertureOffsetCm.X, ApertureOffsetCm.Y, 0.0f);

		// Initialize bounds
		OutLeft   =  FLT_MAX;
		OutRight  = -FLT_MAX;
		OutBottom =  FLT_MAX;
		OutTop    = -FLT_MAX;

		// We shoot rays from ApertureSamplePoint (aperture offset in camera view space) toward focus corners (in the focus plane)
		// and find where they hit the near plane
		for (int32 CornerIdx = 0; CornerIdx < 4; ++CornerIdx)
		{
			// Ray from ApertureSamplePoint toward focus plane corner
			const FVector FrustumEdgeRay = FocusCorners[CornerIdx] - ApertureSamplePoint;

			// Check for parallel ray (perpendicular to near plane)
			if (FMath::Abs(FrustumEdgeRay.Z) < KINDA_SMALL_NUMBER)
			{
				// Ray is parallel to near plane - skip this corner
				continue;
			}

			// Intersect with near plane Z = NearCm
			const float T = (NearCm - ApertureSamplePoint.Z) / FrustumEdgeRay.Z;
			const FVector NearPoint = ApertureSamplePoint + T * FrustumEdgeRay;

			const FVector NearPointRelativeToSamplePoint = NearPoint - ApertureSamplePoint;

			OutLeft   = FMath::Min(OutLeft  , NearPointRelativeToSamplePoint.X);
			OutRight  = FMath::Max(OutRight , NearPointRelativeToSamplePoint.X);
			OutBottom = FMath::Min(OutBottom, NearPointRelativeToSamplePoint.Y);
			OutTop    = FMath::Max(OutTop   , NearPointRelativeToSamplePoint.Y);
		}

		if (OutLeft >= OutRight || OutBottom >= OutTop)
		{
			OutLeft   = -NearCm;
			OutRight  =  NearCm;
			OutBottom = -NearCm;
			OutTop    =  NearCm;
		}
	}

	FMatrix BuildOffAxisProjectionMatrix(
		float Left,
		float Right,
		float Bottom,
		float Top,
		float NearPlane,
		float FarPlane
	)
	{
		// Based on DisplayCluster's MakeProjectionMatrix implementation
		// This creates an asymmetric frustum

		const float n = NearPlane;
		const float f = FarPlane;
		const float r = Right;
		const float l = Left;
		const float t = Top;
		const float b = Bottom;

		// Check for degenerate frustum bounds
		if (FMath::Abs(r - l) < KINDA_SMALL_NUMBER || FMath::Abs(t - b) < KINDA_SMALL_NUMBER)
		{
			// Return identity matrix for degenerate case
			return FMatrix::Identity;
		}

		// Build the matrix
		//
		//     2n/(r-l)         0             0          0
		//
		//       0            2n/(t-b)        0          0
		//
		// -(r+l)/(r-l)   -(t+b)/(t-b)       n/(n-f)     1
		//
		//       0              0          -fn/(n-f)     0


		const float mx = 2.0f * n / (r - l);
		const float my = 2.0f * n / (t - b);

		// Off-axis terms that shift the principal point
		const float ma = -(r + l) / (r - l);
		const float mb = -(t + b) / (t - b);

		// Reversed-Z depth
		const float mc = FMath::IsNearlyEqual(f, n) ? 0 :      n / (n - f);
		const float md = FMath::IsNearlyEqual(f, n) ? n : -f * n / (n - f);
		const float me = 1.0f;

		FMatrix Result(
			FPlane(mx,  0,  0,  0),
			FPlane( 0, my,  0,  0),
			FPlane(ma, mb, mc, me),
			FPlane( 0,  0, md,  0)
		);

		return Result;
	}

	void ReorderSamplesForPendulum(TArray<FVector2f>& InOutOffsets)
	{
		// This preserves the first sample (expect center) and ends close to it which benefits temporal reprojection continuity

		const int32 N = InOutOffsets.Num();

		if (N <= 2)
		{
			return;
		}

		TArray<FVector2f> Reordered;
		Reordered.Reserve(N);

		// First pass: every other sample (0, 2, 4, ...)
		for (int32 Idx = 0; Idx < N; Idx += 2)
		{
			Reordered.Add(InOutOffsets[Idx]);
		}

		// Second pass: skipped samples in reverse (..., 5, 3, 1)
		const int32 LastOddIdx = (N % 2 == 0) ? (N - 1) : (N - 2);
		for (int32 Idx = LastOddIdx; Idx >= 1; Idx -= 2)
		{
			Reordered.Add(InOutOffsets[Idx]);
		}

		InOutOffsets = MoveTemp(Reordered);
	}

	FVector2f GenerateHaltonJitter(int32 SampleIndex)
	{
		const float HaltonX = Halton(SampleIndex, 2);
		const float HaltonY = Halton(SampleIndex, 3);
		return FVector2f(HaltonX - 0.5f, HaltonY - 0.5f);
	}

	FVector2f GenerateSobolJitter(int32 SampleIndex, uint32 RotationSeed)
	{
		// Split seed
		const int32 SeedX = static_cast<int32>(RotationSeed & 0xFFFFFF);
		const int32 SeedY = static_cast<int32>((RotationSeed >> 8) & 0xFFFFFF);

		// Note: FSobol::Evaluate returns [0, 1)

		const float SobolX = FSobol::Evaluate(SampleIndex, 0, SeedX);
		const float SobolY = FSobol::Evaluate(SampleIndex, 1, SeedY);

		return FVector2f(SobolX - 0.5f, SobolY - 0.5f);
	}

	float MitchellNetravali1D(float Distance)
	{
		// Mitchell-Netravali cubic filter with B = C = 1/3
		// See https://en.wikipedia.org/wiki/Mitchell–Netravali_filters for more info.

		constexpr float B = 1.0f / 3.0f;
		constexpr float C = 1.0f / 3.0f;

		const float AbsD = FMath::Abs(Distance);

		if (AbsD < 1.0f)
		{
			constexpr float Q0_0 = (6.0f - 2.0f * B) / 6.0f;
			constexpr float Q0_2 = (-18.0f + 12.0f * B + 6.0f * C) / 6.0f;
			constexpr float Q0_3 = (12.0f - 9.0f * B - 6.0f * C) / 6.0f;

			return Q0_0 + AbsD * AbsD * (Q0_2 + AbsD * Q0_3);
		}
		else if (AbsD < 2.0f)
		{
			constexpr float Q1_0 = (8.0f * B + 24.0f * C) / 6.0f;
			constexpr float Q1_1 = (-12.0f * B - 48.0f * C) / 6.0f;
			constexpr float Q1_2 = (6.0f * B + 30.0f * C) / 6.0f;
			constexpr float Q1_3 = (-B - 6.0f * C) / 6.0f;

			return Q1_0 + AbsD * (Q1_1 + AbsD * (Q1_2 + AbsD * Q1_3));
		}

		return 0.0f;
	}

}
