// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "Math/Matrix.h"

/**
 * Math utilities for cinematic thin-lens depth of field calculations.
 */
namespace AccumulationDOFMath
{
	/** Inward margin applied to aperture radius when generating sample positions.
	 *  Prevents outer ring samples from landing exactly on the boundary where
	 *  they may fail validity tests. Expressed as fraction of radius (0.02 = 2%). */
	constexpr float ApertureSampleMargin = 0.02f;

	/**
	 * Number of samples added per ring increment.
	 * 
	 * Ring i has (i+1) * RingSamplesIncrement samples.
	 * Ideally 2*pi (~6.28) for equal radial/tangential spacing, using 6 as closest integer. 
	 */
	constexpr int32 RingSamplesIncrement = 6;

	/**
	 * Golden angle in radians for inter-ring rotation useful in breaking potential radial patterns.
	 * 
	 * Comes from the golden ratio phi = (1+sqrt(5))/2 = L/S, with L + S = 1 (the circle normalized to unit circumference)
	 * We use the shorter segment S = 1/phi^2. The rotation angle being that fraction of 2*pi radians:
	 *     2*pi * (1/phi^2)
	 *     = 137.507... deg, or 2.399... rad.
	 */
	constexpr float GoldenAngle = 2.39996322972865332f;

	/** Convert millimeters to centimeters */
	FORCEINLINE float MmToCm(float Mm)
	{
		return Mm * 0.1f;
	}

	/** Convert centimeters to millimeters */
	FORCEINLINE float CmToMm(float Cm)
	{
		return Cm * 10.0f;
	}

	/** Calculate aperture radius in cm from focal length (mm) and f-stop */
	FORCEINLINE float GetApertureRadiusCm(float FocalLengthMm, float FStop)
	{
		// D_mm = FocalLengthMm / FStop
		// R_cm = 0.5 * D_mm * 0.1  (converts to cm)

		const float SafeFStop = FMath::Max(FStop, KINDA_SMALL_NUMBER);
		return 0.05f * (FocalLengthMm / SafeFStop);
	}

	/**
	 * Compute spherical aberration focus distance shift (additive) for a given aperture sample.
	 * Based on primary spherical aberration model: dZ = -(4 / (n' * NA^2)) * W040 * rho^2
	 *
	 * @param ApertureOffsetCm - 2D offset on the aperture (in cm)
	 * @param ApertureRadiusCm - Physical aperture radius (in cm)
	 * @param FocalLengthCm    - Focal length (in cm)
	 * @param SphericalAberrationCm - Primary spherical aberration coefficient (Seidel W040) in cm
	 *
	 * @return Additive shift to apply to focus distance
	 */
	FORCEINLINE float ComputeSphericalAberrationShift(
		const FVector2f& ApertureOffsetCm,
		float ApertureRadiusCm,
		float FocalLengthCm,
		float SphericalAberrationCm)
	{
		if (FMath::IsNearlyZero(SphericalAberrationCm) || ApertureRadiusCm <= KINDA_SMALL_NUMBER || FocalLengthCm <= KINDA_SMALL_NUMBER)
		{
			return 0.0f;
		}

		// Normalized pupil coordinate rho (0 at center, 1 at edge)
		const float OffsetMagnitudeCm = FMath::Sqrt(ApertureOffsetCm.X * ApertureOffsetCm.X + ApertureOffsetCm.Y * ApertureOffsetCm.Y);
		const float Rho = FMath::Clamp(OffsetMagnitudeCm / ApertureRadiusCm, 0.0f, 1.0f);
		const float RhoSquared = Rho * Rho;

		// NA = aperture_radius / focal_length (paraxial approximation in air)
		const float NA = ApertureRadiusCm / FocalLengthCm;
		const float NASquared = FMath::Max(NA * NA, UE_KINDA_SMALL_NUMBER);

		// n_image = 1.0 (air)
		const float n_image = 1.0f;

		// dZ = -(4 / (n' * NA^2)) * W040 * rho^2
		// Positive W040 -> negative shift (marginal rays focus closer than paraxial)
		const float DeltaZCm = -(4.0f / (n_image * NASquared)) * SphericalAberrationCm * RhoSquared;

		return DeltaZCm;
	}

	/**
	 * Compute the f-stop value for DOFSplats.
	 * Uses a constant f-stop for all samples based on the specified fraction of the main aperture.
	 *
	 * @param ApertureRadiusCm - Physical main aperture radius (in cm)
	 * @param FocalLengthMm    - Focal length in mm (needed for f-stop calculation)
	 * @param BaseFraction     - Fraction of main aperture diameter for DOFSplats (e.g., 0.125 = 1/8th, 1.0 = full)
	 * 
	 * @return F-stop value for DOFSplats, or 0 if DOFSplats should be disabled
	 */
	FORCEINLINE float ComputeDOFSplatsFStop(
		float ApertureRadiusCm,
		float FocalLengthMm,
		float BaseFraction)
	{
		if (BaseFraction <= 0.0f || ApertureRadiusCm <= KINDA_SMALL_NUMBER || FocalLengthMm <= KINDA_SMALL_NUMBER)
		{
			return 0.0f;
		}

		// Compute splat aperture diameter in mm
		const float MainApertureDiameterMm = CmToMm(ApertureRadiusCm * 2.0f);
		const float SplatDiameterMm = MainApertureDiameterMm * BaseFraction;

		if (SplatDiameterMm <= KINDA_SMALL_NUMBER)
		{
			return 0.0f;
		}

		// F-stop = focal_length / aperture_diameter
		return FocalLengthMm / SplatDiameterMm;
	}

	/**
	 * Generate Halton sequence value for given index and base. Index is 0-based.
	 */
	float Halton(int32 Index, int32 Base);

	/**
	 * Map uniform [0,1) samples to a unit disk using Shirley-Chiu concentric disk mapping
	 * Returns point on disk with radius <= 1.0
	 */
	FVector2f ConcentricDiskSample(float U, float V);

	/**
	 * Generate aperture sample offsets using Halton sequence for thin-lens camera.
	 * Samples are mapped to a unit disk using Shirley-Chiu concentric disk mapping.
	 * 
	 * @param NumSamples       - Number of samples to generate
	 * @param ApertureRadiusCm - Aperture radius in cm
	 * @param OutOffsets       - Output array of 2D offsets
	 */
	void GenerateApertureSamplesHalton(int32 NumSamples, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets);

	/**
	 * Generate aperture sample offsets using concentric ring pattern.
	 * Samples are arranged in rings with staggered angular offsets between rings to maximize
	 * minimum distance between samples.
	 *
	 * @param NumRings         - Number of rings (0 = center only, 1 = 1 + RingSamplesIncrement samples, 2 = 1 + 2*RingSamplesIncrement samples, etc.)
	 * @param ApertureRadiusCm - Aperture radius in cm
	 * @param OutOffsets       - Output array of 2D offsets
	 */
	void GenerateApertureSamplesRing(int32 NumRings, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets);

	/**
	 * Generate aperture sample offsets using Vogel spiral
	 *
	 * @param NumSamples       - Number of samples to generate (uses exact count)
	 * @param ApertureRadiusCm - Aperture radius in cm
	 * @param OutOffsets       - Output array of 2D offsets
	 */
	void GenerateApertureSamplesVogel(int32 NumSamples, float ApertureRadiusCm, TArray<FVector2f>& OutOffsets);

	/**
	 * Compute the number of rings needed to achieve at least the desired sample count.
	 */
	int32 ComputeRingCountForSamples(int32 DesiredSamples);

	/**
	 * Compute total sample count for a given number of rings (includes center sample).
	 * Ring i has (i+1)*RingSamplesIncrement samples, so sum = RingSamplesIncrement*(1+2+...+N) = RingSamplesIncrement*N*(N+1)/2
	 */
	FORCEINLINE int32 GetRingSampleCount(int32 NumRings)
	{
		if (NumRings <= 0)
		{
			return 1;
		}

		const int64 Rings64 = static_cast<int64>(NumRings);
		const int64 Samples64 = 1 + (static_cast<int64>(RingSamplesIncrement) * Rings64 * (Rings64 + 1)) / 2;
		return static_cast<int32>(FMath::Min(Samples64, static_cast<int64>(MAX_int32)));
	}

	/**
	 * Compute sensor corners in camera local space
	 * 
	 * @param SensorWidthCm  - Sensor width in cm
	 * @param SensorHeightCm - Sensor height in cm
	 * @param FocalLengthCm  - Focal length in cm
	 * @param OutCorners     - Output array of 4 corner positions
	 */
	void ComputeSensorCorners(float SensorWidthCm, float SensorHeightCm, float FocalLengthCm, FVector OutCorners[4]);

	/**
	 * Compute focus plane corners by intersecting rays from aperture center through sensor corners
	 * with the focus plane at FocusDistanceCm
	 * 
	 * @param SensorCorners   - 4 sensor corners in camera local space
	 * @param FocusDistanceCm - Focus distance in cm
	 * @param OutFocusCorners - Output array of 4 focus plane corners
	 */
	void ComputeFocusPlaneCorners(const FVector SensorCorners[4], float FocusDistanceCm, FVector OutFocusCorners[4]);

	/**
	 * Compute off-axis frustum bounds at the near plane for a given aperture sample offset
	 * 
	 * @param FocusCorners     - 4 corners of the focus plane quad
	 * @param ApertureOffsetCm - 2D offset on the lens, in cm
	 * @param NearCm           - Near clipping plane distance
	 * @params OutLeft, OutRight, OutBottom, OutTop - Frustum bounds at near plane
	 */
	void ComputeOffAxisFrustum(
		const FVector FocusCorners[4],
		const FVector2f& ApertureOffsetCm,
		float NearCm,
		float& OutLeft,
		float& OutRight,
		float& OutBottom,
		float& OutTop
	);

	/**
	 * Build an off-axis perspective projection matrix.
	 */
	FMatrix BuildOffAxisProjectionMatrix(
		float Left,
		float Right,
		float Bottom,
		float Top,
		float NearPlane,
		float FarPlane
	);

	/**
	 * Reorder samples in a pendulum (forth and back) pattern to minimize perspective jumps between frames
	 * and also start and end near the center of the lens.
	 *
	 * @param InOutOffsets - Array of aperture offsets for in-place reordering.
	 */
	void ReorderSamplesForPendulum(TArray<FVector2f>& InOutOffsets);

	/**
	 * Generate Halton 2D jitter using bases 2 and 3 for AA.
	 *
	 * @param SampleIndex - Sample index (0-based)
	 * 
	 * @return 2D jitter in range [-0.5, 0.5]
	 */
	FVector2f GenerateHaltonJitter(int32 SampleIndex);

	/**
	 * Generate Sobol 2D jitter with Cranley-Patterson rotation for AA.
	 * Uses a constant seed for temporal stability across frames.
	 *
	 * @param SampleIndex  - Sample index (0-based)
	 * @param RotationSeed - Seed for Cranley-Patterson rotation
	 * 
	 * @return 2D jitter in range [-0.5, 0.5]
	 */
	FVector2f GenerateSobolJitter(int32 SampleIndex, uint32 RotationSeed = 0x12345678);

	/**
	 * Mitchell-Netravali 1D reconstruction filter
	 * Support range: [-2, 2], returns 0 outside.
	 *
	 * @param Distance - Distance from filter center in pixels
	 * 
	 * @return Filter weight
	 */
	float MitchellNetravali1D(float Distance);

	/**
	 * Separable 2D Mitchell-Netravali filter.
	 *
	 * @param OffsetPixels - 2D offset from filter center in pixel units
	 * 
	 * @return Filter weight
	 */
	FORCEINLINE float MitchellNetravali2D(const FVector2f& OffsetPixels)
	{
		return MitchellNetravali1D(OffsetPixels.X) * MitchellNetravali1D(OffsetPixels.Y);
	}
}
