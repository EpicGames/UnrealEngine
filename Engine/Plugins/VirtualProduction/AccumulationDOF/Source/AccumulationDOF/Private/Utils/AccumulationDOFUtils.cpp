// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFUtils.h"

#include "AccumulationDOFMath.h"
#include "CineCameraComponent.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "HAL/IConsoleManager.h"
#include "RenderingThread.h"

DEFINE_LOG_CATEGORY_STATIC(LogAccumulationDOFUtils, Log, All);

namespace AccumulationDOFUtils
{

//
// Bokeh Shape Calculation
//

FBokehShapeParams ComputeBokehShape(float FStop, float MinFstop, int32 BladeCountRaw)
{
	FBokehShapeParams Result;

	const int32 ClampedBladeCount = FMath::Clamp(BladeCountRaw, 4, 16);
	Result.DiaphragmBladeCount = ClampedBladeCount;

	constexpr float CircumscribedRadius = 1.0f;
	constexpr float TargetedBokehArea = PI * CircumscribedRadius * CircumscribedRadius;

	if (FStop <= MinFstop)
	{
		// Circle: aperture wide open, blades not visible
		Result.BokehShape = 0;
		Result.DiaphragmBladeCount = 0;
		Result.CocRadiusToCircumscribedRadius = 1.0f;
		Result.CocRadiusToIncircleRadius = 1.0f;
		Result.DiaphragmRotationRad = 0.0f;
		Result.DiaphragmBladeRadius = 0.0f;
		Result.DiaphragmBladeCenterOffset = 0.0f;
	}
	else if (MinFstop <= UE_KINDA_SMALL_NUMBER)
	{
		// StraightBlades: maximum aperture, no blade curvature
		Result.BokehShape = 1;

		const float BladeCoverageAngle = PI / ClampedBladeCount;
		const float TriangleArea = CircumscribedRadius * CircumscribedRadius * FMath::Cos(BladeCoverageAngle) * FMath::Sin(BladeCoverageAngle);
		const float CircleRadius = FMath::Sqrt(ClampedBladeCount * TriangleArea / TargetedBokehArea);

		Result.CocRadiusToCircumscribedRadius = CircumscribedRadius / CircleRadius;
		Result.CocRadiusToIncircleRadius = Result.CocRadiusToCircumscribedRadius * FMath::Cos(BladeCoverageAngle);
		Result.DiaphragmRotationRad = 0.0f;
		Result.DiaphragmBladeRadius = 0.0f;
		Result.DiaphragmBladeCenterOffset = 0.0f;
	}
	else
	{
		// RoundedBlades: partially retracted blades with curved edges
		Result.BokehShape = 2;

		const float BladeCoverageAngle = PI / ClampedBladeCount;
		const float BladeRadius = CircumscribedRadius * FStop / MinFstop;
		const float AsinArg = FMath::Clamp((CircumscribedRadius / BladeRadius) * FMath::Sin(BladeCoverageAngle), -1.0f, 1.0f);
		const float BladeVisibleAngle = FMath::Asin(AsinArg);
		const float BladeCircleOffset = BladeRadius * FMath::Cos(BladeVisibleAngle) - CircumscribedRadius * FMath::Cos(BladeCoverageAngle);

		// Area computations for energy conservation
		const float InscribedTriangleArea = CircumscribedRadius * CircumscribedRadius * FMath::Cos(BladeCoverageAngle) * FMath::Sin(BladeCoverageAngle);
		const float BladeInscribedTriangleArea = BladeRadius * BladeRadius * FMath::Cos(BladeVisibleAngle) * FMath::Sin(BladeVisibleAngle);
		const float AdditionalCircleArea = PI * BladeRadius * BladeRadius * (BladeVisibleAngle / PI) - BladeInscribedTriangleArea;
		const float InscribedBokehArea = ClampedBladeCount * (InscribedTriangleArea + AdditionalCircleArea);
		const float UpscaleFactor = FMath::Sqrt(TargetedBokehArea / InscribedBokehArea);

		// Blade pivot for rotation
		const float BladePivotCenterX = 0.5f * (BladeRadius - CircumscribedRadius);
		const float BladePivotCenterY = FMath::Sqrt(BladeRadius * BladeRadius - BladePivotCenterX * BladePivotCenterX);

		Result.DiaphragmRotationRad = FMath::Atan2(BladePivotCenterX, BladePivotCenterY);
		Result.DiaphragmBladeRadius = UpscaleFactor * BladeRadius;
		Result.DiaphragmBladeCenterOffset = UpscaleFactor * BladeCircleOffset;
		Result.CocRadiusToCircumscribedRadius = UpscaleFactor * CircumscribedRadius;
		Result.CocRadiusToIncircleRadius = UpscaleFactor * (BladeRadius - BladeCircleOffset);
	}

	return Result;
}

//
// Camera Parameter Extraction
//

FCameraParams ExtractCameraParams(const UCineCameraComponent* CineCam)
{
	FCameraParams Result;

	if (!CineCam)
	{
		return Result;
	}

	// Core lens parameters
	Result.FocalLengthMm    = CineCam->CurrentFocalLength;
	Result.FocalLengthCm    = AccumulationDOFMath::MmToCm(Result.FocalLengthMm);
	Result.FStop            = CineCam->CurrentAperture;
	Result.ApertureRadiusCm = AccumulationDOFMath::GetApertureRadiusCm(Result.FocalLengthMm, Result.FStop);

	// Sensor dimensions
	Result.SensorWidthMm    = CineCam->Filmback.SensorWidth;
	Result.SensorHeightMm   = CineCam->Filmback.SensorHeight;
	Result.SensorWidthCm    = AccumulationDOFMath::MmToCm(Result.SensorWidthMm);
	Result.SensorHeightCm   = AccumulationDOFMath::MmToCm(Result.SensorHeightMm);

	// Get merged PostProcessSettings via GetCameraView
	FMinimalViewInfo CameraViewInfo;
	const_cast<UCineCameraComponent*>(CineCam)->GetCameraView(0.0f, CameraViewInfo);
	const FPostProcessSettings& PP = CameraViewInfo.PostProcessSettings;

	// Anamorphic squeeze - read from merged PP (includes asymmetric overscan adjustment)
	// GetCameraView() copies LensSettings.SqueezeFactor to PP.DepthOfFieldSqueezeFactor
	Result.SqueezeFactor = PP.bOverride_DepthOfFieldSqueezeFactor
		? FMath::Max(PP.DepthOfFieldSqueezeFactor, UE_KINDA_SMALL_NUMBER)
		: 1.0f;

	// Focus distance
	Result.FocusDistanceCm = PP.bOverride_DepthOfFieldFocalDistance
		? PP.DepthOfFieldFocalDistance
		: CineCam->CurrentFocusDistance;

	// Petzval field curvature
	Result.Petzval = PP.bOverride_DepthOfFieldPetzvalBokeh
		? PP.DepthOfFieldPetzvalBokeh
		: 0.0f;

	Result.PetzvalFalloffPower = PP.bOverride_DepthOfFieldPetzvalBokehFalloff
		? FMath::Clamp(PP.DepthOfFieldPetzvalBokehFalloff, 0.0f, 100.0f)
		: 1.0f;

	Result.PetzvalExclusionBoxExtents = PP.bOverride_DepthOfFieldPetzvalExclusionBoxExtents
		? PP.DepthOfFieldPetzvalExclusionBoxExtents
		: FVector2f::ZeroVector;

	Result.PetzvalExclusionBoxRadius = PP.bOverride_DepthOfFieldPetzvalExclusionBoxRadius
		? PP.DepthOfFieldPetzvalExclusionBoxRadius
		: 0.0f;

	// Cat's eye (PP values assumed to be in cm to match engine behavior)
	Result.BarrelRadiusCm = PP.bOverride_DepthOfFieldBarrelRadius
		? PP.DepthOfFieldBarrelRadius
		: 0.0f;

	Result.BarrelLengthCm = PP.bOverride_DepthOfFieldBarrelLength
		? PP.DepthOfFieldBarrelLength
		: 0.0f;

	// Clamp BarrelRadiusCm to minimum (ApertureRadiusCm + 0.5) like DiaphragmDOF
	if (Result.BarrelLengthCm > UE_KINDA_SMALL_NUMBER)
	{
		Result.BarrelRadiusCm = FMath::Max(Result.BarrelRadiusCm, Result.ApertureRadiusCm + 0.5f);
	}

	// Diaphragm blade settings for bokeh shape computation
	Result.BladeCountRaw = PP.bOverride_DepthOfFieldBladeCount ? PP.DepthOfFieldBladeCount : 4;
	Result.MinFstop = PP.bOverride_DepthOfFieldMinFstop ? PP.DepthOfFieldMinFstop : 0.0f;

	return Result;
}

//
// LUT Size Override (because the default 32x32x32 can cause banding on HDR signals)
//

int32 OverrideLUTSizeIfNeeded(bool bShouldOverride)
{
	if (!bShouldOverride)
	{
		return 0;
	}

	static IConsoleVariable* CVarLUTSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LUT.Size"));

	if (!CVarLUTSize)
	{
		return 0;
	}

	const int32 CurrentLUTSize = CVarLUTSize->GetInt();

	if (CurrentLUTSize < 64)
	{
		CVarLUTSize->Set(64, ECVF_SetByCode);
		UE_LOGF(LogAccumulationDOFUtils, Verbose, "Overriding r.LUT.Size from %d to 64 to reduce banding", CurrentLUTSize);
		return CurrentLUTSize;
	}

	return 0;
}

void RestoreLUTSize(int32 PreviousSize)
{
	if (PreviousSize <= 0)
	{
		return;
	}

	static IConsoleVariable* CVarLUTSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LUT.Size"));

	if (CVarLUTSize)
	{
		CVarLUTSize->Set(PreviousSize, ECVF_SetByCode);
		UE_LOGF(LogAccumulationDOFUtils, Verbose, "Restored r.LUT.Size to %d", PreviousSize);
	}
}

void ReleaseRenderingMemory()
{
	if (GEngine)
	{ 
		GEngine->ForceGarbageCollection(true);
	}
	
	// @todo is there a GPU equivalent ?

}

}
