// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UCineCameraComponent;
class UTextureRenderTarget2D;

struct FPostProcessSettings;


namespace AccumulationDOFUtils
{
	/**
	 * Output structure for diaphragm bokeh model computation.
	 * Used to render realistic aperture blade shapes.
	 */
	struct FBokehShapeParams
	{
		int32 BokehShape = 0;                       // 0:Circle, 1:StraightBlades, 2:RoundedBlades
		int32 DiaphragmBladeCount = 0;              // 0 for circle, 4-16 for bladed aperture
		float DiaphragmRotationRad = 0.0f;          // Rotation angle for blade orientation
		float CocRadiusToCircumscribedRadius = 1.0f;
		float CocRadiusToIncircleRadius = 1.0f;
		float DiaphragmBladeRadius = 0.0f;          // Blade arc radius (for rounded blades)
		float DiaphragmBladeCenterOffset = 0.0f;    // Blade arc center offset (for rounded blades)
	};

	/**
	 * Compute bokeh shape parameters from f-stop and diaphragm settings.
	 * Implements the same logic as DiaphragmDOF::FBokehModel::Compile.
	 *
	 * @param FStop         - Current f-stop value
	 * @param MinFstop      - Minimum f-stop before blades become visible
	 * @param BladeCountRaw - Raw blade count (4-16, from PP settings)
	 *
	 * @return FBokehShapeParams with computed values
	 */
	FBokehShapeParams ComputeBokehShape(float FStop, float MinFstop, int32 BladeCountRaw);


	/**
	 * Camera and lens parameters extracted from CineCameraComponent and PostProcessSettings.
	 * Consolidates all camera-related values needed for DOF accumulation.
	 */
	struct FCameraParams
	{
		// Core lens parameters
		float FocalLengthMm = 0.0f;
		float FocalLengthCm = 0.0f;
		float FStop = 2.8f;
		float ApertureRadiusCm = 0.0f;
		float FocusDistanceCm = 0.0f;

		// Sensor dimensions
		float SensorWidthMm = 0.0f;
		float SensorHeightMm = 0.0f;
		float SensorWidthCm = 0.0f;
		float SensorHeightCm = 0.0f;

		// Anamorphic
		float SqueezeFactor = 1.0f;

		// Petzval field curvature (from PostProcessSettings)
		float Petzval = 0.0f;
		float PetzvalFalloffPower = 1.0f;
		FVector2f PetzvalExclusionBoxExtents = FVector2f::ZeroVector;
		float PetzvalExclusionBoxRadius = 0.0f;

		// Barrel occlusion (cat's eye bokeh)
		float BarrelRadiusCm = 0.0f;
		float BarrelLengthCm = 0.0f;

		// Diaphragm blade settings (raw values for bokeh shape computation)
		int32 BladeCountRaw = 4;
		float MinFstop = 0.0f;
	};

	/**
	 * Extract camera parameters from CineCameraComponent.
	 * Reads lens settings and merged PostProcessSettings.
	 *
	 * @param CineCam - The CineCameraComponent to extract parameters from.
	 *
	 * @return FCameraParams with all relevant values
	 */
	FCameraParams ExtractCameraParams(const UCineCameraComponent* CineCam);

	/**
	 * Override engine color grading LUT size to 64 if currently smaller.
	 * Returns the original value so it can be restored later.
	 *
	 * @param bShouldOverride - Whether to apply the override
	 *
	 * @return Original LUT size value (0 if no override was applied)
	 */
	int32 OverrideLUTSizeIfNeeded(bool bShouldOverride);

	/**
	 * Restore the engine color grading LUT size to a previous value.
	 *
	 * @param PreviousSize - The LUT size to restore (from OverrideLUTSizeIfNeeded)
	 */
	void RestoreLUTSize(int32 PreviousSize);

	/** Number of samples to render before releasing rendering memory to prevent OOM. */
	static constexpr int32 RenderingMemoryFlushBatchSize = 256;

	/** Number of samples between GPU flushes for UI responsiveness during one-shot preview. */
	static constexpr int32 UIFlushBatchSize = 8;

	/**
	 * Releases rendering memory to prevent accumulation during long render loops.
	 */
	void ReleaseRenderingMemory();

}
