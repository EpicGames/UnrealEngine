// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FAccumulationDOFModule : public IModuleInterface
{
public:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** The name that will be given to the DOF pass that's provided to the pass factory in MRG's deferred renderer. */
	static const FName MRGPassFactoryPassName;
};
