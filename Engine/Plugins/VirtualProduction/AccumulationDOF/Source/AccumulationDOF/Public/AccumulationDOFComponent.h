// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AccumulationDOFTypes.h"

#include "Components/ActorComponent.h"

#include "AccumulationDOFComponent.generated.h"

class UTexture2D;

/**
 * Settings component for cinematic aperture-sampled depth of field.
 *
 * Attach to a CineCameraActor to configure DOF rendering settings for:
 * - Movie Render Queue (via AccumulationDOF pass)
 * - Editor viewport preview (Level Viewport >> Scalability & Performance >> Accumulation DOF)
 */
UCLASS(Blueprintable, ClassGroup = (Rendering), meta = (BlueprintSpawnableComponent, DisplayName = "Accumulation DOF"))
class ACCUMULATIONDOF_API UAccumulationDOFComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UAccumulationDOFComponent()
	{
		bAutoActivate = true;
	}

	/**
	 * Specifies the number of aperture samples to render (use more for better quality, but will take more time).
	 * The actual number of rendered samples may be different in order to comply with the sampling pattern.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field", meta = (ClampMin = "1", UIMin = "1", UIMax = "32768"))
	int32 NumSamples = 256;

	/**
	 * Fraction of main aperture diameter for DOF splat size (0.125 = 1/8th, 1.0 = full aperture, 0 = disabled).
	 * DOF splats add small engine DOF blur to each sample to fill gaps between sparse samples.
	 * This reduces the number of samples needed for a smooth bokeh, but will be softer at the edges than ideal.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field",
		meta = (DisplayName = "DOF Splat Size", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DOFSplatSize = 0.125f;

	/** Enable motion blur on the accumulated DOF result. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field")
	bool bEnableMotionBlur = false;

	/** Aperture sampling pattern. Hexaweb is generally preferred but Vogel Spirals are pretty good too. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field", AdvancedDisplay)
	EApertureSamplingPattern SamplingPattern = EApertureSamplingPattern::Hexaweb;

	/** Controls how temporal history is updated during aperture sampling. Prefer Last for now */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field", AdvancedDisplay)
	ETemporalHistoryMode TemporalHistoryMode = ETemporalHistoryMode::LastSampleOnly;

	/** Use Halton jitter AA instead of per-sample TAA. When disabled, TAA runs on each sample. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field", AdvancedDisplay)
	bool bUseJitterAA = true;

	//
	// Bokeh Settings
	//

	/** Bokeh texture to weight aperture samples. Texture should be linear. If it has an alpha channel, you can optionally pick it as transmission mask. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field|Bokeh")
	TObjectPtr<UTexture2D> BokehTexture = nullptr;

	/** Enable bokeh texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field|Bokeh")
	bool bEnableBokehTexture = true;

	/** Which channel to extract weight from bokeh texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field|Bokeh", meta = (EditCondition = "BokehTexture != nullptr"))
	EBokehWeightChannel WeightChannel = EBokehWeightChannel::Luminance;

	/** Strength of bokeh tint application 0.0 - 1.0 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field|Bokeh", meta = (EditCondition = "BokehTexture != nullptr", ClampMin = "0.0", ClampMax = "1.0"))
	float TintStrength = 0.0f;

	/** Softness of bokeh edges 0.0 - 1.0. Higher values create smoother bokeh boundaries by applying a soft falloff near the aperture edge. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation Depth of Field|Bokeh", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float BokehEdgeSoftness = 0.15f;

	//
	// Aberrations
	//

	/**
	 * Axial chromatic aberration intensity as percentage of focus distance in %.
	 * Note: Lateral chromatic aberration uses the camera's PostProcessSettings.SceneFringeIntensity.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chromatic Aberration", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "100.0"))
	float AxialChromaticAberrationIntensity = 0.0f;

	/**
	 * Number of focal planes for axial chromatic aberration.
	 * Controls how many distinct focal distances are used across the spectrum.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chromatic Aberration",
		meta = (ClampMin = "3", ClampMax = "19", UIMin = "3", UIMax = "19", EditCondition = "AxialChromaticAberrationIntensity > 0"))
	int32 AxialChromaticAberrationNumBands = 6;

	/**
	 * Spectral (actually multi-band) lateral chromatic aberration
	 *
	 * When disabled, uses the Engine's default lateral chromatic aberration.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chromatic Aberration")
	bool bSpectralLateralChromaticAberration = true;

	/**
	 * Primary spherical aberration coefficient (Seidel W040) in cm.
	 * Positive values cause marginal rays to focus closer than paraxial rays.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Monochromatic Aberrations", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "100.0"))
	float SphericalAberration = 0.0f;

	/**
	 * Coma aberration strength (Seidel W131).
	 * Off-axis points develop comet-like tails pointing away from the optical axis.
	 * The effect increases with distance from the image center.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Monochromatic Aberrations", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	float ComaAberration = 0.0f;
};
