// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

ACCUMULATIONDOF_API DECLARE_LOG_CATEGORY_EXTERN(LogAccumulationDOF, Log, All);
