// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AccumulationDOFTypes.generated.h"

/**
 * Options for what texture channel(s) to use to weigh the rgb values during accumulation (i.e. a transmission mask).
 * Note: This is independent of whether the bokeh tints the rgb values or not.
*/
UENUM()
enum class EBokehWeightChannel : uint8
{
	Alpha     UMETA(DisplayName = "Alpha Channel"),
	Luminance UMETA(DisplayName = "RGB Luminance")
};

/** Aperture sampling pattern */
UENUM()
enum class EApertureSamplingPattern : uint8
{
	/** Halton sequence with Shirley-Chiu disk mapping.
	 *
	 * Will use the sample count specified.
	 *
	 * Note: Shirley-Chiu produced more visually consistent results than culling the Halton square to the aperture circle.
	 */
	Halton UMETA(DisplayName = "Halton"),

	/** Concentric rings with staggered angular offsets. Sample distances are roughly maintained (both radially and tangentially).
	 *
	 * Actual number of samples will be picked based on completed ring sample counts, so may not use the exact number of samples
	 * requested.
	 *
	 * Note: Produces more even bokeh compared to Halton.
	 */
	Hexaweb UMETA(DisplayName = "Golden Hexaweb"),

	/** Vogel spiral
	 *
	 * Similar quality results as with Golden Hexaweb
	 */
	Vogel UMETA(DisplayName = "Vogel Spiral")
};

/** Controls how temporal history (e.g. Lumen/TAA) is updated during aperture sampling. */
UENUM()
enum class ETemporalHistoryMode : uint8
{
	/** All samples update history. */
	AllSamplesUpdate UMETA(DisplayName = "All Samples Update"),

	/** Only last sample updates history. */
	LastSampleOnly UMETA(DisplayName = "Last Sample Only"),

	/** Only first sample updates history.*/
	FirstSampleOnly UMETA(DisplayName = "First Sample Only"),

	/** No samples update history. Baseline for comparison. */
	NoSamplesUpdate UMETA(DisplayName = "No Samples Update")
};
