// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphRenderLayerSubsystem.h"
#include "Graph/Nodes/MovieGraphModifierNode.h"

#include "MovieGraphAccumulationDOFModifierNode.generated.h"

#define UE_API ACCUMULATIONDOF_API

/** Specifies how the enable state of the Accumulation DOF component will be set. */
UENUM(BlueprintType)
enum class EMovieGraphAccumulationDOFEnableType : uint8
{
	/** Use the current enable state on the camera (ie, no change is made). */
	CameraActorDefault,

	/** Use a custom value that will be used as the enable state. */
	CustomValue
};

/** Specifies if the enable state of the Accumulation DOF component should be set, and if so, what the value of the enable state should be. */
USTRUCT(BlueprintType)
struct FMovieGraphAccumulationDOFEnableState
{
	GENERATED_BODY()

	UE_API FMovieGraphAccumulationDOFEnableState() = default;

	/** How the Accumulation DOF enable state should be treated. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation DOF")
	EMovieGraphAccumulationDOFEnableType Type = EMovieGraphAccumulationDOFEnableType::CameraActorDefault;

	/** If the enable Type is "Custom", this is the value the enable state should be set to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation DOF", meta = (EditCondition = "Type == EMovieGraphAccumulationDOFEnableType::CustomValue"))
	bool Value = false;
};

/** 
 * Modifies the properties on the Accumulation DOF component (on all cameras within the world).
 */
UCLASS(MinimalAPI)
class UMovieGraphAccumulationDOFModifierNode final : public UMovieGraphSettingNode, public IMovieGraphModifierNodeInterface
{
	GENERATED_BODY()

public:
	UE_API UMovieGraphAccumulationDOFModifierNode();

#if WITH_EDITOR
	//~ Begin UMovieGraphNode interface
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UMovieGraphNode interface
#endif

	//~ Begin IMovieGraphModifierNodeInterface interface
	UE_API virtual TArray<UMovieGraphModifierBase*> GetAllModifiers() const override;
	UE_API virtual bool SupportsCollections() const override;
	//~ End IMovieGraphModifierNodeInterface interface

	//~ Begin UMovieGraphSettingNode interface
	UE_API virtual FString GetNodeInstanceName() const override;
	//~ End UMovieGraphSettingNode interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EnableAccumulationDepthOfField : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumSamples : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DOFSplatSize : 1;

	/** 
	 * Whether the Accumulation DOF component should be enabled/disabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation DOF", meta = (EditCondition = "bOverride_EnableAccumulationDepthOfField"))
	FMovieGraphAccumulationDOFEnableState EnableAccumulationDepthOfField;

	/** 
	 * The number of samples that the Accumulation DOF component should use.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation DOF",
		meta = (EditCondition = "bOverride_NumSamples", ClampMin = "1", UIMin = "1", UIMax = "32768"))
	int32 NumSamples = 256;

	/** 
	 * The splat size that the Accumulation DOF component should use.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Accumulation DOF",
		meta = (EditCondition = "bOverride_DOFSplatSize", DisplayName = "DOF Splat Size", ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float DOFSplatSize = 0.125f;

private:
	/** The modifier associated with this node. */
	UPROPERTY(meta = (HideInActiveRenderSettings))
	mutable TObjectPtr<class UMovieGraphAccumulationDOFModifier> Modifier;
};

#undef UE_API
