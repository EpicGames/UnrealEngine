// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Graph/Renderers/MovieGraphDeferredPass.h"

ACCUMULATIONDOF_API DECLARE_LOG_CATEGORY_EXTERN(LogAccumulationDOFMRG, Log, All);

class ACineCameraActor;
class FAccumulationDOFSceneViewExtension;
class UAccumulationDOFComponent;
class UApertureSampler;
class UCineCameraComponent;
class UTexture2D;
class UTextureRenderTarget2D;

namespace AccumulationDOF
{
	struct FApertureSamplerConfig;
	struct FApertureSamplerCameraState;
}

/**
 * Movie Graph deferred render pass for aperture-sampled depth of field.
 *
 * This pass renders multiple aperture samples with off-axis projection matrices and accumulates them 
 * using GPU shaders to produce plausible DOF.
 *
 * Camera parameters (focal length, aperture, focus distance) are read from the CineCameraActor being 
 * rendered by the sequence.
 */
struct ACCUMULATIONDOF_API FMovieGraphAccumulationDOFPass : public UE::MovieGraph::Rendering::FMovieGraphDeferredPass
{
	FMovieGraphAccumulationDOFPass();
	virtual ~FMovieGraphAccumulationDOFPass() override;

protected:
	//~ FMovieGraphDeferredPass Interface
	virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer) override;
	virtual void Teardown() override;
	virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	//~ FMovieGraphImagePassBase Interface
	virtual void UpdateRenderPassTelemetry(FMoviePipelineEndShotRenderTelemetryContext& InOutTelemetry) const override;
	virtual TSharedRef<FSceneViewFamilyContext> CreateSceneViewFamily(const UE::MovieGraph::Rendering::FViewFamilyInitData& InInitData) const override;
	virtual FVector2D ComputeProjectionMatrixJitter(const FVector2D& InDefaultJitter, const FIntPoint& InBackbufferResolution) const override;
	virtual FSceneViewInitOptions CreateViewInitOptions(const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo, FSceneViewFamilyContext* InViewFamily, FSceneViewStateInterface* InViewStateInterface) const override;
	virtual void CalculateProjectionMatrix(UE::MovieGraph::DefaultRenderer::FCameraInfo& InOutCameraInfo, FSceneViewProjectionData& InOutProjectionData, const FIntPoint InBackbufferResolution, const FIntPoint InAccumulatorResolution) const override;
	virtual void ApplyMovieGraphOverridesToSceneView(TSharedRef<FSceneViewFamilyContext> InOutFamily, const UE::MovieGraph::Rendering::FViewFamilyInitData& InInitData, const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;
	virtual FSceneViewStateInterface* GetSceneViewState(UMovieGraphImagePassBaseNode* ParentNodeThisFrame, int32_t TileX, int32_t TileY) override;
	virtual void GetPreviewBannerMessages(TArray<FMovieGraphPreviewBannerMessage>& OutMessages) const override;

private:
	/** Aperture sampler that handles sample rendering and accumulation. GC-protected via AddReferencedObjects. */
	TObjectPtr<UApertureSampler> ApertureSampler = nullptr;

	/** Current sample being rendered */
	mutable int32 CurrentApertureSampleIndex = -1;

	/** Focus distance override for axial CA (negative means no override). */
	mutable float CurrentFocusDistanceOverride = -1.0f;

	/** RGB weight for current spectral band in axial chromatic aberration. */
	FVector3f CurrentAxialCAWeight = FVector3f(1.0f);

	/** Cached reference to CineCameraComponent for SVE settings. */
	TWeakObjectPtr<UCineCameraComponent> CachedCineCameraComponent;

	/** Cached reference to AccumulationDOFComponent (found in CineCameraActor). */
	TWeakObjectPtr<UAccumulationDOFComponent> CachedDOFComponent;

	/** Scene View Extension for capture passes (one per aperture sample). */
	TSharedPtr<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe> CaptureExtension;

	/** Scene View Extension for injection pass. */
	TSharedPtr<FAccumulationDOFSceneViewExtension, ESPMode::ThreadSafe> InjectionExtension;

	/** Flag to indicate we're doing a capture pass (so AddViewExtensions knows to add the capture SVE). */
	bool bInCapturePass = false;

	/** Flag to indicate we're doing the injection pass (so AddViewExtensions knows to add the injection SVE). */
	bool bInInjectionPass = false;

	/** Track whether we've initialized camera-dependent state. */
	bool bCameraParamsInitialized = false;

	/** Cached SceneFringeIntensity from the first sample's SVE (blended with PP volumes). */
	float CachedSceneFringeIntensity = 0.0f;

	/** Separate view state for injection renders, keyed per tile so the injection pass keeps its own temporal history per tile. */
	TMap<FIntPoint, FSceneViewStateReference> InjectionViewStates;

	/** Tracks which per-tile injection view states have had their camera-cut prime applied. */
	TSet<FIntPoint> InjectionStatesInitialized;

	/** Active tile index for the current Render() iteration, used by helper functions. */
	FIntPoint CurrentTileIndex = FIntPoint(0, 0);

	/**
	 * Tile grid locked at Setup time. ADOF does not support mid-shot TileCount changes because the
	 * per-tile injection view state map (InjectionViewStates) is allocated once in Setup.
	 */
	FIntPoint CurrentTileCount = FIntPoint(1, 1);

	/** Tile overlap pad in pixels for the current Render() iteration. */
	FIntPoint CurrentTileOverlappedPad = FIntPoint(0, 0);

	/** Cached motion blur fraction from current frame's sample state. */
	mutable float CachedMotionBlurFraction = 0.0f;

	/** Saved LUT size before override (0 if not overridden). */
	int32 SavedLUTSize = 0;

	/** Effective telemetry values captured while render-layer modifiers are active. */
	int32 CachedActualNumSamples = 0;
	float CachedEffectiveDOFSplatSize = 0.f;

private:
	/** Build sampler config from graph settings and component. */
	AccumulationDOF::FApertureSamplerConfig BuildSamplerConfig(UMovieGraphEvaluatedConfig* InEvaluatedConfig) const;

	/** Build camera state from current sequence camera. */
	AccumulationDOF::FApertureSamplerCameraState BuildCameraState() const;

	/** Initialize the aperture sampler. */
	void InitializeSampler(UMovieGraphEvaluatedConfig* InEvaluatedConfig);

	/** Update camera-dependent state (called each frame). */
	void UpdateCameraDependentState();

	/** Find CineCameraComponent from MRG's camera binding. */
	UCineCameraComponent* FindCineCameraComponent() const;

	/** Find AccumulationDOFComponent on the camera actor. */
	UAccumulationDOFComponent* FindAccumulationDOFComponent();

	/** Get the DOF splat size from the component, or the default if there's no valid component. */
	float GetEffectiveDOFSplatSize() const;

	/** Render all aperture samples in one temporal sample. */
	void RenderSamples(
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData,
		const FResolutionAndCameraInfo& InResolutionInfo,
		const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams);

	/** Render a single aperture sample. */
	void RenderSingleApertureSample(
		int32 SampleIndex,
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData,
		const FResolutionAndCameraInfo& InResolutionInfo,
		const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams);

	/** Injection pass and submit to MRG. */
	void RenderInjectionAndSubmit(
		const FMovieGraphTraversalContext& InFrameTraversalContext,
		const FMovieGraphTimeStepData& InTimeData,
		const FResolutionAndCameraInfo& InResolutionInfo,
		const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams);
};
