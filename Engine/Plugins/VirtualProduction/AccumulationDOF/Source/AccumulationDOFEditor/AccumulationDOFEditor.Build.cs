// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AccumulationDOFEditor : ModuleRules
{
	public AccumulationDOFEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		string RuntimeModuleDir = Path.Combine(ModuleDirectory, "..", "AccumulationDOF");

		PrivateIncludePaths.AddRange(new string[] {
			Path.Combine(RuntimeModuleDir, "Private/Rendering"),
			Path.Combine(RuntimeModuleDir, "Private/Shaders"),
			Path.Combine(RuntimeModuleDir, "Private/Utils"),
		});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AccumulationDOF",
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			}
		);
	}
}
