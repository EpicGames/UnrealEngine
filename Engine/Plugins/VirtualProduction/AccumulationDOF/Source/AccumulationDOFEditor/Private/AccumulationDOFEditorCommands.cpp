// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFEditorCommands.h"

#define LOCTEXT_NAMESPACE "AccumulationDOFEditor"

void FAccumulationDOFEditorCommands::RegisterCommands()
{
	UI_COMMAND(
		ToggleAccumulate,
		"Toggle Accumulate",
		"Toggle accumulation depth of field preview in the active viewport",
		EUserInterfaceActionType::ToggleButton,
		FInputChord(EModifierKey::Alt, EKeys::D)
	);
}

#undef LOCTEXT_NAMESPACE
