// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AccumulationDOFEditor"

class FAccumulationDOFEditorCommands : public TCommands<FAccumulationDOFEditorCommands>
{
public:
	FAccumulationDOFEditorCommands()
		: TCommands<FAccumulationDOFEditorCommands>(
			TEXT("AccumulationDOFEditor"),
			LOCTEXT("AccumulationDOFEditorCommandsContext", "Accumulation DOF"),
			TEXT("LevelViewport"),
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ToggleAccumulate;
};

#undef LOCTEXT_NAMESPACE
