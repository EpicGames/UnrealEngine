// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFEditorModule.h"

#include "AccumulationDOFEditorCommands.h"
#include "AccumulationDOFEditorSettings.h"
#include "AccumulationDOFViewportExtension.h"
#include "AccumulationDOFViewportManager.h"
#include "AccumulationDOFViewportSettings.h"

#include "Editor.h"
#include "Framework/Commands/UICommandList.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "AccumulationDOFEditor"

void FAccumulationDOFEditorModule::StartupModule()
{
	ViewportManager = MakeShared<FAccumulationDOFViewportManager>();

	FAccumulationDOFEditorCommands::Register();

	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FAccumulationDOFEditorModule::OnEngineLoopInitComplete);

	ToolMenusStartupCallbackHandle = UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FAccumulationDOFEditorModule::RegisterViewMenuExtension));
}

void FAccumulationDOFEditorModule::ShutdownModule()
{
	if (ToolMenusStartupCallbackHandle.IsValid())
	{
		UToolMenus::UnRegisterStartupCallback(ToolMenusStartupCallbackHandle);
		ToolMenusStartupCallbackHandle.Reset();
	}

	UnregisterViewMenuExtension();

	BoundCommands.Reset();
	FAccumulationDOFEditorCommands::Unregister();

	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}

	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	ViewportManager.Reset();
}

void FAccumulationDOFEditorModule::RegisterViewMenuExtension()
{
	FToolMenuOwnerScoped ToolMenuOwnerScoped(this);

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolBar.PerformanceAndScalability");

	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection(
		"AccumulationDOF",
		LOCTEXT("AccumulationDOF_SectionLabel", "Accumulation DOF")
	);

	Section.AddDynamicEntry(
		NAME_None,
		FNewToolMenuSectionDelegate::CreateRaw(this, &FAccumulationDOFEditorModule::AddAccumulationDOFControls)
	);
}

void FAccumulationDOFEditorModule::UnregisterViewMenuExtension()
{
	if (UObjectInitialized())
	{
		UToolMenus::UnregisterOwner(this);
	}
}

void FAccumulationDOFEditorModule::AddAccumulationDOFControls(FToolMenuSection& InSection)
{
	TSharedPtr<SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(InSection);

	if (!LevelViewport.IsValid())
	{
		return;
	}

	TWeakPtr<SLevelViewport> WeakViewport = LevelViewport;

	// Accumulate toggle
	InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		FName("AccumulateDOF"),
		LOCTEXT("AccumulateDOF_Label", "Accumulate"),
		TAttribute<FText>::CreateLambda([]() -> FText
		{
			const FAccumulationDOFEditorCommands& Commands = FAccumulationDOFEditorCommands::Get();

			if (Commands.ToggleAccumulate.IsValid())
			{
				FText InputText = Commands.ToggleAccumulate->GetInputText();
				if (!InputText.IsEmpty())
				{
					return FText::Format(LOCTEXT("AccumulateDOF_Tooltip_WithShortcut", "Enable accumulation depth of field preview ({0})"), InputText);
				}
			}

			return LOCTEXT("AccumulateDOF_Tooltip_NoShortcut", "Enable accumulation depth of field preview (shortcut unset)");
		}),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, WeakViewport]()
			{
				TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

				if (!Viewport.IsValid())
				{
					return;
				}

				ToggleAccumulationForViewport(&Viewport->GetLevelViewportClient());
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, WeakViewport]()
			{
				TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

				if (!Viewport.IsValid())
				{
					return false;
				}

				return IsAccumulationEnabledForViewport(&Viewport->GetLevelViewportClient());
			})),
		EUserInterfaceActionType::ToggleButton
	));

	// One-shot / Unfreeze button
	FToolMenuEntry OneshotEntry = FToolMenuEntry::InitMenuEntry(
		FName("OneshotDOF"),
		TAttribute<FText>::CreateLambda([this, WeakViewport]() -> FText
		{
			TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

			if (!Viewport.IsValid())
			{
				return FText::GetEmpty();
			}

			FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();

			if (TSharedPtr<FAccumulationDOFViewportExtension> Extension = ViewportManager->GetViewportExtension(ViewportClient))
			{
				if (Extension->IsFrozen())
				{
					return LOCTEXT("UnfreezeDOF_Label", "Unfreeze");
				}
			}

			return LOCTEXT("OneshotDOF_Label", "One-shot");
		}),
		TAttribute<FText>::CreateLambda([this, WeakViewport]() -> FText
		{
			TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

			if (!Viewport.IsValid())
			{
				return FText::GetEmpty();
			}

			FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();

			if (TSharedPtr<FAccumulationDOFViewportExtension> Extension = ViewportManager->GetViewportExtension(ViewportClient))
			{
				if (Extension->IsFrozen())
				{
					return LOCTEXT("UnfreezeDOF_Tooltip", "Unfreeze the accumulated DOF result");
				}
			}

			return LOCTEXT("OneshotDOF_Tooltip", "Capture a single high-quality DOF frame");
		}),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, WeakViewport]()
			{
				TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

				if (!Viewport.IsValid())
				{
					return;
				}

				FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();

				if (TSharedPtr<FAccumulationDOFViewportExtension> Extension = ViewportManager->GetViewportExtension(ViewportClient))
				{
					if (Extension->IsFrozen())
					{
						ViewportManager->Unfreeze(ViewportClient);

						return;
					}
				}

				ViewportManager->CaptureOneshot(ViewportClient);
			})),
		EUserInterfaceActionType::Button
	);

	InSection.AddEntry(OneshotEntry);

	// Settings submenu
	InSection.AddSubMenu(
		"AccumulationDOFSettings",
		LOCTEXT("AccumulationDOFSettings_Label", "Settings"),
		LOCTEXT("AccumulationDOFSettings_ToolTip", "Sample count and splat size settings"),
		FNewToolMenuDelegate::CreateRaw(this, &FAccumulationDOFEditorModule::AddAccumulationDOFSettingsSubMenu),
		false
	);
}

void FAccumulationDOFEditorModule::AddAccumulationDOFSettingsSubMenu(UToolMenu* Menu)
{
	Menu->bSearchable = false;

	FToolMenuSection& Section = Menu->AddDynamicSection(
		"AdvancedSettings",
		FNewToolMenuDelegate::CreateLambda([this](UToolMenu* DynamicMenu)
		{
			TSharedPtr<SLevelViewport> LevelViewport = ULevelViewportContext::GetLevelViewport(DynamicMenu);

			if (!LevelViewport.IsValid())
			{
				return;
			}

			FLevelEditorViewportClient* ViewportClient = &LevelViewport->GetLevelViewportClient();
			ViewportManager->FindOrAddViewportSettings(ViewportClient);
			TWeakPtr<SLevelViewport> WeakViewport = LevelViewport;
			FToolMenuSection& SettingsSection = DynamicMenu->AddSection("AdvancedDOFSettings");

			// Use Camera Settings checkbox
			SettingsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
				FName("UseCameraSettings"),
				LOCTEXT("UseCameraSettings_Label", "Use Camera Settings"),
				LOCTEXT("UseCameraSettings_Tooltip", "Use the number of samples and DOF splat size specified the cine camera's AccumulationDOF component, if there is one."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, WeakViewport]()
					{
						TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

						if (!Viewport.IsValid())
						{
							return;
						}

						FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
						FAccumulationDOFViewportSettings& Settings = ViewportManager->FindOrAddViewportSettings(ViewportClient);
						Settings.bUseCameraSettings = !Settings.bUseCameraSettings;
						EnsureViewportTrackedForPersistence(ViewportClient, Viewport);
						ViewportManager->RestartAccumulation(ViewportClient);

						if (ViewportClient->Viewport)
						{
							ViewportClient->Viewport->Invalidate();
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, WeakViewport]()
					{
						TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

						if (!Viewport.IsValid())
						{
							return false;
						}

						FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
						const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

						return Settings && Settings->bUseCameraSettings;
					})),
				EUserInterfaceActionType::ToggleButton
			));

			// Number of samples
			SettingsSection.AddEntry(FToolMenuEntry::InitWidget(
				"NumSamplesWidget",
				SNew(SSpinBox<int32>)
				.MinValue(1)
				.MaxValue(16384)
				.IsEnabled_Lambda([this, WeakViewport]()
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return true;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

					return !Settings || !Settings->bUseCameraSettings;
				})
				.Value_Lambda([this, WeakViewport]()
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return 256;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

					return Settings ? Settings->NumApertureSamples : 256;
				})
				.OnValueCommitted_Lambda([this, WeakViewport](int32 NewValue, ETextCommit::Type)
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					FAccumulationDOFViewportSettings& Settings = ViewportManager->FindOrAddViewportSettings(ViewportClient);
					Settings.NumApertureSamples = FMath::Clamp(NewValue, 1, 16384);
					EnsureViewportTrackedForPersistence(ViewportClient, Viewport);
					ViewportManager->RestartAccumulation(ViewportClient);

					if (ViewportClient->Viewport)
					{
						ViewportClient->Viewport->Invalidate();
					}
				}),
				LOCTEXT("NumSamplesLabel", "Samples   ")
			));

			// DOF Splat Size
			SettingsSection.AddEntry(FToolMenuEntry::InitWidget(
				"SplatSizeWidget",
				SNew(SSpinBox<float>)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Delta(0.01f)
				.IsEnabled_Lambda([this, WeakViewport]()
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return true;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

					return !Settings || !Settings->bUseCameraSettings;
				})
				.Value_Lambda([this, WeakViewport]()
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return 0.125f;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

					return Settings ? Settings->DOFSplatSize : 0.125f;
				})
				.OnValueCommitted_Lambda([this, WeakViewport](float NewValue, ETextCommit::Type)
				{
					TSharedPtr<SLevelViewport> Viewport = WeakViewport.Pin();

					if (!Viewport.IsValid())
					{
						return;
					}

					FLevelEditorViewportClient* ViewportClient = &Viewport->GetLevelViewportClient();
					FAccumulationDOFViewportSettings& Settings = ViewportManager->FindOrAddViewportSettings(ViewportClient);
					Settings.DOFSplatSize = FMath::Clamp(NewValue, 0.0f, 1.0f);
					EnsureViewportTrackedForPersistence(ViewportClient, Viewport);
					ViewportManager->RestartAccumulation(ViewportClient);

					if (ViewportClient->Viewport)
					{
						ViewportClient->Viewport->Invalidate();
					}
				}),
				LOCTEXT("SplatSizeLabel", "Splat Size")
			));
		}));
}

void FAccumulationDOFEditorModule::OnLevelViewportClientListChanged()
{
	if (!GEditor)
	{
		return;
	}

	const TArray<FLevelEditorViewportClient*> LevelViewportClients = GEditor->GetLevelViewportClients();

	// Clean up viewports that no longer exist from ConfiguredViewports tracking

	for (auto ViewportIt = ConfiguredViewports.CreateIterator(); ViewportIt; ++ViewportIt)
	{
		FViewportPair& ViewportData = *ViewportIt;

		bool bViewportStillExists = LevelViewportClients.ContainsByPredicate([ViewportData](const FLevelEditorViewportClient* Other)
		{
			return Other == ViewportData.Key;
		});

		if (bViewportStillExists)
		{
			continue;
		}

		// Viewport was removed, save settings and clean up
		const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportData.Key);

		if (Settings)
		{
			GetMutableDefault<UAccumulationDOFLevelViewportSettings>()->SetViewportSettings(ViewportData.Value, *Settings);
			GetMutableDefault<UAccumulationDOFLevelViewportSettings>()->SaveConfig();
			ViewportManager->RemoveViewportSettings(ViewportData.Key);
		}

		ViewportIt.RemoveCurrent();
	}

	// Also cleanup any extensions for viewports not in ConfiguredViewports
	ViewportManager->RemoveInvalidViewports(LevelViewportClients);

	// Set up new viewports
	for (FLevelEditorViewportClient* LevelViewportClient : LevelViewportClients)
	{
		SetupViewportDisplaySettings(LevelViewportClient);
	}
}

void FAccumulationDOFEditorModule::OnEngineLoopInitComplete()
{
	if (GEditor)
	{
		GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FAccumulationDOFEditorModule::OnLevelViewportClientListChanged);
	}

	BindCommands();
}

void FAccumulationDOFEditorModule::SetupViewportDisplaySettings(FLevelEditorViewportClient* Client)
{
	bool bAlreadyConfigured = ConfiguredViewports.ContainsByPredicate([Client](const FViewportPair& Other)
	{
		return Other.Key == Client;
	});

	if (bAlreadyConfigured)
	{
		return;
	}

	TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(Client->GetEditorViewportWidget());

	if (!LevelViewport.IsValid())
	{
		return;
	}

	TOptional<FAccumulationDOFViewportSettings> PresetConfig =
		GetDefault<UAccumulationDOFLevelViewportSettings>()->GetViewportSettings(LevelViewport->GetConfigKey());

	if (!PresetConfig.IsSet())
	{
		return;
	}

	FAccumulationDOFViewportSettings& ViewportConfig = ViewportManager->FindOrAddViewportSettings(Client);
	ViewportConfig = PresetConfig.GetValue();

	FViewportPair NewEntry(Client, LevelViewport->GetConfigKey());
	ConfiguredViewports.Emplace(MoveTemp(NewEntry));
}

void FAccumulationDOFEditorModule::EnsureViewportTrackedForPersistence(FLevelEditorViewportClient* Client, TSharedPtr<SLevelViewport> LevelViewport)
{
	if (!Client || !LevelViewport.IsValid())
	{
		return;
	}

	bool bAlreadyTracked = ConfiguredViewports.ContainsByPredicate([Client](const FViewportPair& Other)
	{
		return Other.Key == Client;
	});

	if (bAlreadyTracked)
	{
		return;
	}

	FViewportPair NewEntry(Client, LevelViewport->GetConfigKey());
	ConfiguredViewports.Emplace(MoveTemp(NewEntry));
}

void FAccumulationDOFEditorModule::ToggleAccumulationForViewport(FLevelEditorViewportClient* ViewportClient)
{
	if (!ViewportClient)
	{
		return;
	}

	TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(ViewportClient->GetEditorViewportWidget());
	FAccumulationDOFViewportSettings& Settings = ViewportManager->FindOrAddViewportSettings(ViewportClient);
	Settings.bIsEnabled = !Settings.bIsEnabled;

	if (LevelViewport.IsValid())
	{
		EnsureViewportTrackedForPersistence(ViewportClient, LevelViewport);
	}

	ViewportManager->Unfreeze(ViewportClient);

	if (ViewportClient->Viewport)
	{
		ViewportClient->Viewport->Invalidate();
	}
}

bool FAccumulationDOFEditorModule::IsAccumulationEnabledForViewport(FLevelEditorViewportClient* ViewportClient) const
{
	if (!ViewportClient)
	{
		return false;
	}

	const FAccumulationDOFViewportSettings* Settings = ViewportManager->GetViewportSettings(ViewportClient);

	return Settings && Settings->bIsEnabled;
}

void FAccumulationDOFEditorModule::BindCommands()
{
	if (!FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	const TSharedRef<FUICommandList>& GlobalActions = LevelEditorModule.GetGlobalLevelEditorActions();

	const FAccumulationDOFEditorCommands& Commands = FAccumulationDOFEditorCommands::Get();

	BoundCommands = MakeShared<FUICommandList>();
	BoundCommands->MapAction(
		Commands.ToggleAccumulate,
		FExecuteAction::CreateLambda([]()
		{
			if (FAccumulationDOFEditorModule* Module = FModuleManager::Get().GetModulePtr<FAccumulationDOFEditorModule>("AccumulationDOFEditor"))
			{
				Module->ToggleAccumulationForViewport(GCurrentLevelEditingViewportClient);
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]()
		{
			if (FAccumulationDOFEditorModule* Module = FModuleManager::Get().GetModulePtr<FAccumulationDOFEditorModule>("AccumulationDOFEditor"))
			{
				return Module->IsAccumulationEnabledForViewport(GCurrentLevelEditingViewportClient);
			}

			return false;
		})
	);

	GlobalActions->Append(BoundCommands.ToSharedRef());
}

IMPLEMENT_MODULE(FAccumulationDOFEditorModule, AccumulationDOFEditor)

#undef LOCTEXT_NAMESPACE
