// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FAccumulationDOFViewportManager;
class FLevelEditorViewportClient;
class FUICommandList;
class UToolMenu;
struct FAccumulationDOFViewportSettings;
struct FToolMenuSection;

/**
 * Editor module for AccumulationDOF viewport integration.
 * Provides DOF preview via the scalability dropdown menu of each editor level viewport.
 */
class FAccumulationDOFEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	/** Get the viewport manager. Only valid after StartupModule() and before ShutdownModule(). */
	FAccumulationDOFViewportManager& GetViewportManager()
	{
		check(ViewportManager.IsValid());
		return *ViewportManager;
	}

	/** Get this module's instance */
	static FAccumulationDOFEditorModule& Get()
	{
		return FModuleManager::GetModuleChecked<FAccumulationDOFEditorModule>("AccumulationDOFEditor");
	}

private:

	/** Register the View menu extension */
	void RegisterViewMenuExtension();

	/** Unregister the View menu extension */
	void UnregisterViewMenuExtension();

	/** Add Accumulate checkbox and One-shot button to the section */
	void AddAccumulationDOFControls(FToolMenuSection& InSection);

	/** Build the settings submenu (Samples, Splat Size) */
	void AddAccumulationDOFSettingsSubMenu(UToolMenu* Menu);

	/** Called when level viewport client list changes */
	void OnLevelViewportClientListChanged();

	/** Called when engine loop initialization is complete */
	void OnEngineLoopInitComplete();

	/** Setup viewport display settings from saved config */
	void SetupViewportDisplaySettings(FLevelEditorViewportClient* Client);

	/** Ensure viewport is tracked for settings persistence on removal */
	void EnsureViewportTrackedForPersistence(FLevelEditorViewportClient* Client, TSharedPtr<class SLevelViewport> LevelViewport);

	/** Toggle accumulation for a specific viewport client */
	void ToggleAccumulationForViewport(FLevelEditorViewportClient* ViewportClient);

	/** Check if accumulation is enabled for a specific viewport client */
	bool IsAccumulationEnabledForViewport(FLevelEditorViewportClient* ViewportClient) const;

	/** Bind keyboard shortcut commands */
	void BindCommands();

private:

	/** Manages per-viewport DOF configurations and extensions */
	TSharedPtr<FAccumulationDOFViewportManager> ViewportManager;

	/** Track configured viewports for cleanup */
	using FViewportPair = TPair<FLevelEditorViewportClient*, FName>;

	/** List of configured level editor viewports */
	TArray<FViewportPair> ConfiguredViewports;

	/** Delegate handle for ToolMenus startup callback */
	FDelegateHandle ToolMenusStartupCallbackHandle;

	/** Command list for keyboard shortcuts */
	TSharedPtr<FUICommandList> BoundCommands;
};
