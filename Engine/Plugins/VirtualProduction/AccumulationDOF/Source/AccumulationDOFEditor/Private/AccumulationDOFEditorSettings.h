// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "AccumulationDOFViewportSettings.h"

#include "AccumulationDOFEditorSettings.generated.h"

/**
 * Pair of viewport identifier and its settings.
 */
USTRUCT()
struct FAccumulationDOFPerViewportSettingsPair
{
	GENERATED_BODY()

	/** Viewport identifier */
	UPROPERTY(config)
	FName ViewportIdentifier;

	/** Whether DOF is enabled */
	UPROPERTY(config)
	bool bIsEnabled = false;

	/** Use settings from camera's AccumulationDOFComponent */
	UPROPERTY(config)
	bool bUseCameraSettings = false;

	/** Number of aperture samples */
	UPROPERTY(config)
	int32 NumApertureSamples = 256;

	/** Splats size (aperture diameter fraction) */
	UPROPERTY(config)
	float DOFSplatSize = 0.125f;

	/** Samples to render per frame */
	UPROPERTY(config)
	int32 SamplesPerFrame = 2;

	/** Convert to FAccumulationDOFViewportSettings with validated ranges */
	FAccumulationDOFViewportSettings ToSettings() const
	{
		FAccumulationDOFViewportSettings Settings;

		Settings.bIsEnabled         = bIsEnabled;
		Settings.bUseCameraSettings = bUseCameraSettings;
		Settings.NumApertureSamples = FMath::Max(1, NumApertureSamples);
		Settings.DOFSplatSize       = FMath::Clamp(DOFSplatSize, 0.0f, 1.0f);
		Settings.SamplesPerFrame    = FMath::Max(1, SamplesPerFrame);

		return Settings;
	}

	/** Convert from FAccumulationDOFViewportSettings */
	void FromSettings(const FAccumulationDOFViewportSettings& Settings)
	{
		bIsEnabled         = Settings.bIsEnabled;
		bUseCameraSettings = Settings.bUseCameraSettings;
		NumApertureSamples = Settings.NumApertureSamples;
		DOFSplatSize       = Settings.DOFSplatSize;
		SamplesPerFrame    = Settings.SamplesPerFrame;
	}
};

/**
 * Saved per-viewport settings for Accumulation DOF.
 */
UCLASS(config = AccumulationDOF)
class UAccumulationDOFLevelViewportSettings : public UObject
{
	GENERATED_BODY()

public:

	/** Settings array */
	UPROPERTY(config)
	TArray<FAccumulationDOFPerViewportSettingsPair> ViewportsSettings;

	/** Get settings for a viewport if they exist */
	TOptional<FAccumulationDOFViewportSettings> GetViewportSettings(FName ViewportIdentifier) const
	{
		if (ViewportIdentifier.IsNone())
		{
			return TOptional<FAccumulationDOFViewportSettings>();
		}

		const FAccumulationDOFPerViewportSettingsPair* Found = ViewportsSettings.FindByPredicate(
			[ViewportIdentifier](const FAccumulationDOFPerViewportSettingsPair& Pair)
			{
				return Pair.ViewportIdentifier == ViewportIdentifier;
			});

		if (Found)
		{
			return Found->ToSettings();
		}

		return TOptional<FAccumulationDOFViewportSettings>();
	}

	/** Set settings for a viewport */
	void SetViewportSettings(FName ViewportIdentifier, const FAccumulationDOFViewportSettings& Settings)
	{
		if (ViewportIdentifier.IsNone())
		{
			return;
		}

		FAccumulationDOFPerViewportSettingsPair* Found = ViewportsSettings.FindByPredicate(
			[ViewportIdentifier](const FAccumulationDOFPerViewportSettingsPair& Pair)
			{
				return Pair.ViewportIdentifier == ViewportIdentifier;
			});

		if (Found)
		{
			Found->FromSettings(Settings);
		}
		else
		{
			FAccumulationDOFPerViewportSettingsPair NewPair;
			NewPair.ViewportIdentifier = ViewportIdentifier;
			NewPair.FromSettings(Settings);
			ViewportsSettings.Add(NewPair);
		}
	}
};
