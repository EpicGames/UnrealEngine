// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFViewportExtension.h"

#include "AccumulationDOFComponent.h"
#include "AccumulationDOFLog.h"
#include "AccumulationDOFShaders.h"
#include "ApertureSampler.h"

#include "Camera/CameraComponent.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "TextureResource.h"
#include "LevelEditorViewport.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ShowFlags.h"
#include "UnrealClient.h"
#include "UObject/UObjectGlobals.h"


static TAutoConsoleVariable<int32> CVarAccumulationDOFShowPreviewLabel(
	TEXT("r.AccumulationDOF.ShowPreviewLabel"),
	1,
	TEXT("Show 'Accumulation DOF Preview' label in viewport preview.\n")
	TEXT("  0: Hide label\n")
	TEXT("  1: Show label (default)"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarAccumulationDOFPreviewScreenPercentageMin(
	TEXT("r.AccumulationDOF.Preview.ScreenPercentage.Min"),
	100,
	TEXT("Minimum screen percentage for Accumulation DOF preview rendering.\n")
	TEXT("Uses the greater of this value or the viewport screen percentage."),
	ECVF_Default
);


// Compute camera's filmback aspect ratio.
static float GetCameraFilmbackAspectRatio(const UCineCameraComponent* CineCam)
{
	if (!CineCam)
	{
		return 0.0f;
	}

	const float SensorWidth   = CineCam->Filmback.SensorWidth;
	const float SensorHeight  = CineCam->Filmback.SensorHeight;
	const float SqueezeFactor = CineCam->LensSettings.SqueezeFactor;

	// Guard against division by zero
	if (SensorHeight <= UE_KINDA_SMALL_NUMBER)
	{
		return 0.0f;
	}

	// Desqueezed aspect ratio
	float FilmbackAspectRatio = (SensorWidth * SqueezeFactor) / SensorHeight;

	// Apply crop aspect ratio
	if (CineCam->CropSettings.AspectRatio > 0.0f)
	{
		FilmbackAspectRatio = CineCam->CropSettings.AspectRatio;
	}

	return FilmbackAspectRatio;
}


FAccumulationDOFViewportExtension::FAccumulationDOFViewportExtension(
	const FAutoRegister& AutoRegister,
	FLevelEditorViewportClient* AssociatedViewportClient)
	: FSceneViewExtensionBase(AutoRegister)
	, LinkedViewportClient(AssociatedViewportClient)
{
	// Subscribe to scene change delegates for invalidation detection
	if (GEngine)
	{
		OnActorMovedHandle = GEngine->OnActorMoved().AddRaw(
			this, &FAccumulationDOFViewportExtension::OnSceneActorMoved
		);

		OnLevelActorAddedHandle = GEngine->OnLevelActorAdded().AddRaw(
			this, &FAccumulationDOFViewportExtension::OnSceneLevelActorAdded
		);

		OnLevelActorDeletedHandle = GEngine->OnLevelActorDeleted().AddRaw(
			this, &FAccumulationDOFViewportExtension::OnSceneLevelActorDeleted
		);

		OnComponentTransformChangedHandle = GEngine->OnComponentTransformChanged().AddRaw(
			this, &FAccumulationDOFViewportExtension::OnSceneComponentTransformChanged
		);
	}

	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(
		this, &FAccumulationDOFViewportExtension::OnSceneObjectPropertyChanged
	);
}

FAccumulationDOFViewportExtension::~FAccumulationDOFViewportExtension()
{
	// Unsubscribe from scene change delegates (if not already unbound by InvalidateViewportClient)
	if (GEngine)
	{
		if (OnActorMovedHandle.IsValid())
		{
			GEngine->OnActorMoved().Remove(OnActorMovedHandle);
		}

		if (OnLevelActorAddedHandle.IsValid())
		{
			GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
		}

		if (OnLevelActorDeletedHandle.IsValid())
		{
			GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		}

		if (OnComponentTransformChangedHandle.IsValid())
		{
			GEngine->OnComponentTransformChanged().Remove(OnComponentTransformChangedHandle);
		}
	}

	if (OnObjectPropertyChangedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
	}

	if (ApertureSampler)
	{
		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}
}

void FAccumulationDOFViewportExtension::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (ApertureSampler)
	{
		Collector.AddReferencedObject(ApertureSampler);
	}
}

bool FAccumulationDOFViewportExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	if (!Context.Viewport || LinkedViewportClient != Context.Viewport->GetClient())
	{
		return false;
	}

	// Active when enabled OR when frozen (to display one-shot result even when not enabled)
	if (!Settings.bIsEnabled && !bIsFrozen)
	{
		return false;
	}

	// Only active when piloting a CineCameraActor. The idea here is that if the user is not piloting one,
	// be able to edit the scene without DOF without having to disable our Accumulation DOF feature every time.

	if (!GetPilotedCineCameraActor())
	{
		return false;
	}

	return true;
}

void FAccumulationDOFViewportExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	bIsViewModeCompatible = IsViewModeCompatible(InView);

	// Detect view mode changes and restart accumulation
	EViewModeIndex NewViewMode = FindViewMode(InView.Family->EngineShowFlags);
	if (NewViewMode != CachedViewMode)
	{
		CachedViewMode = NewViewMode;
		IncrementInvalidationCounter();
	}

	// Skip if incompatible view mode or not active (enabled or frozen)
	if (!bIsViewModeCompatible || (!Settings.bIsEnabled && !bIsFrozen))
	{
		return;
	}

	// Cache the view rect. Includes camera aspect ratio constraints applied.
	CachedViewRect = InView.UnscaledViewRect;

	// Disable engine DOF
	const_cast<FSceneViewFamily*>(InView.Family)->EngineShowFlags.SetDepthOfField(false);

	// Disable engine lateral chromatic aberration when using our spectral version.

	bool bUseSpectralLateralCA = true;

	if (UAccumulationDOFComponent* DOFComp = GetPilotedDOFComponent())
	{
		bUseSpectralLateralCA = DOFComp->bSpectralLateralChromaticAberration;
	}

	if (bUseSpectralLateralCA)
	{
		InView.FinalPostProcessSettings.SceneFringeIntensity = 0.0f;
	}
}

void FAccumulationDOFViewportExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	// Skip if not active (enabled or frozen) or incompatible view mode
	if ((!Settings.bIsEnabled && !bIsFrozen) || !bIsViewModeCompatible)
	{
		return;
	}

	// Only tick once per engine frame
	uint64 CurrentFrame = GFrameCounter;

	if (CurrentFrame == LastTickFrame)
	{
		return;
	}

	LastTickFrame = CurrentFrame;

	// Check for Sequencer-driven camera changes while frozen
	CheckFrozenCameraStateChanged();

	TickAmortizedRendering();

	// Prepare preview texture
	if (ApertureSampler)
	{
		CachedPreviewTexture   = ApertureSampler->PreparePreviewTexture();
		CachedProgressFraction = ApertureSampler->GetProgress().GetProgressFraction();
		bCachedIsComplete      = ApertureSampler->IsComplete();

		if (UTextureRenderTarget2D* PreviewRT = CachedPreviewTexture.Get())
		{
			if (FTextureRenderTargetResource* Resource = PreviewRT->GameThread_GetRenderTargetResource())
			{
				CachedPreviewRHITexture_RenderThread = Resource->GetRenderTargetTexture();
			}
			else
			{
				CachedPreviewRHITexture_RenderThread = nullptr;
			}
		}
		else
		{
			CachedPreviewRHITexture_RenderThread = nullptr;
		}
	}

	bCachedIsFrozen = bIsFrozen;
}

void FAccumulationDOFViewportExtension::SubscribeToPostProcessingPass(
	EPostProcessingPass Pass,
	const FSceneView& View,
	FAfterPassCallbackDelegateArray& InOutPassCallbacks,
	bool bIsPassEnabled)
{
	// Skip if not active (enabled or frozen) or incompatible view mode
	if ((!Settings.bIsEnabled && !bIsFrozen) || !bIsViewModeCompatible)
	{
		return;
	}

	if (Pass == EPostProcessingPass::MotionBlur)
	{
		InOutPassCallbacks.Add(
			FAfterPassCallbackDelegate::CreateRaw(this, &FAccumulationDOFViewportExtension::ProcessAtMotionBlurPass_RenderThread));
	}
}

void FAccumulationDOFViewportExtension::TickAmortizedRendering()
{
	// Skip amortized rendering when frozen or not enabled
	// (One-shot capture handles its own rendering path)
	if (bIsFrozen || !Settings.bIsEnabled)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (!World)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "TickAmortizedRendering: No world");
		return;
	}

	// Check for scene invalidation

	const bool bSceneChanged = (SceneInvalidationCounter != LastSeenInvalidationCounter);

	LastSeenInvalidationCounter = SceneInvalidationCounter;

	if (bSceneChanged && ApertureSampler && ApertureSampler->IsInitialized())
	{
		ApertureSampler->ResetAccumulation();
	}

	// Use view rect size

	FIntPoint RenderResolution = CachedViewRect.Size();

	if (RenderResolution.X <= 0 || RenderResolution.Y <= 0)
	{
		UE_LOGF(LogAccumulationDOF, Warning, "TickAmortizedRendering: Invalid resolution %dx%d", RenderResolution.X, RenderResolution.Y);
		return;
	}

	UCineCameraComponent* CineCam = GetPilotedCineCameraComponent();

	// Adjust for camera's actual filmback aspect ratio when constrained.
	if (CineCam && CineCam->bConstrainAspectRatio)
	{
		const float CameraAspect = GetCameraFilmbackAspectRatio(CineCam);
		if (CameraAspect > 0.0f)
		{
			RenderResolution.X = FMath::RoundToInt(RenderResolution.Y * CameraAspect);
		}
	}

	// Apply overscan scaling to resolution

	float OverscanFraction = 0.0f;

	if (CineCam)
	{
		OverscanFraction = CineCam->Overscan;

		if (OverscanFraction > 0.0f)
		{
			const float OverscanScale = 1.0f + OverscanFraction;
			RenderResolution.X = FMath::CeilToInt(RenderResolution.X * OverscanScale);
			RenderResolution.Y = FMath::CeilToInt(RenderResolution.Y * OverscanScale);
		}
	}

	// Cache overscan for progress bar positioning
	CachedOverscanFraction = (CineCam && CineCam->bCropOverscan) ? OverscanFraction : 0.0f;

	// Check if we need to reinitialize due to resolution, world, or DOF settings change

	bool bNeedsReinit = !ApertureSampler || !ApertureSampler->IsInitialized();

	if (ApertureSampler && CachedViewportResolution != RenderResolution)
	{
		bNeedsReinit = true;
	}

	if (CachedWorld.Get() != World)
	{
		bNeedsReinit = true;
		bIsFrozen = false; // Unfreeze on world change (intentional reset)
	}

	if (bDOFSettingsNeedReinit)
	{
		bNeedsReinit = true;
		bDOFSettingsNeedReinit = false;
	}

	// Build camera state

	AccumulationDOF::FApertureSamplerCameraState CameraState = BuildCameraStateFromViewport();

	if (!CameraState.bInitialized)
	{
		return;
	}

	if (bNeedsReinit)
	{
		// Build configuration

		AccumulationDOF::FApertureSamplerConfig Config;

		Config.World               = World;
		Config.Resolution          = RenderResolution;
		Config.NumSamples          = Settings.NumApertureSamples;
		Config.DOFSplatSize        = Settings.DOFSplatSize;
		Config.SamplesPerFrame     = Settings.SamplesPerFrame;
		Config.Mode                = AccumulationDOF::EApertureSamplerMode::Amortized;
		Config.TemporalHistoryMode = ETemporalHistoryMode::AllSamplesUpdate; // @todo Need to fix velocity buffer affecting temporal reprojection on first sample

		// Query screen percentage (max of CVar minimum and viewport)
		const float MinScreenPercentage = static_cast<float>(CVarAccumulationDOFPreviewScreenPercentageMin.GetValueOnGameThread()) / 100.0f;
		float ViewportScreenPercentage = 1.0f;

		if (FEditorViewportClient* EditorClient = LinkedViewportClient)
		{
			ViewportScreenPercentage = EditorClient->IsPreviewingScreenPercentage()
				? EditorClient->GetPreviewScreenPercentage() / 100.0f
				: EditorClient->GetDefaultPrimaryResolutionFractionTarget();
		}

		Config.ScreenPercentage = FMath::Max(MinScreenPercentage, ViewportScreenPercentage);

		// Set overscan from camera
		Config.OverscanFraction = OverscanFraction;

		// Use AccumulationDOFComponent settings if available

		if (UAccumulationDOFComponent* DOFComp = GetPilotedDOFComponent())
		{
			// Use camera component's NumSamples and DOFSplatSize when bUseCameraSettings is enabled
			if (Settings.bUseCameraSettings)
			{
				Config.NumSamples   = DOFComp->NumSamples;
				Config.DOFSplatSize = DOFComp->DOFSplatSize;
			}

			Config.SamplingPattern                     = DOFComp->SamplingPattern;
			Config.AxialChromaticAberrationIntensity   = DOFComp->AxialChromaticAberrationIntensity;
			Config.AxialChromaticAberrationNumBands    = DOFComp->AxialChromaticAberrationNumBands;
			Config.bSpectralLateralChromaticAberration = DOFComp->bSpectralLateralChromaticAberration;
			Config.SphericalAberration                 = DOFComp->SphericalAberration;
			Config.ComaAberration                      = DOFComp->ComaAberration;
			Config.BokehTexture                        = DOFComp->BokehTexture;
			Config.bEnableBokehTexture                 = DOFComp->bEnableBokehTexture;
			Config.WeightChannel                       = DOFComp->WeightChannel;
			Config.TintStrength                        = DOFComp->TintStrength;
			Config.BokehEdgeSoftness                   = DOFComp->BokehEdgeSoftness;
			Config.bUseJitterAA                        = DOFComp->bUseJitterAA;
		}

		Config.ViewModeIndex = CachedViewMode;

		// Share exposure state with main viewport to prevent exposure divergence
		if (FEditorViewportClient* EditorClient = LinkedViewportClient)
		{
			Config.ExposureViewState = EditorClient->ViewState.GetReference();
		}

		if (!ApertureSampler)
		{
			ApertureSampler = NewObject<UApertureSampler>(GetTransientPackage());
		}
		else
		{
			ApertureSampler->Shutdown();
		}

		if (!ApertureSampler->Initialize(Config, CameraState))
		{
			return;
		}

		CachedViewportResolution = RenderResolution;
		CachedWorld = World;
	}
	else
	{
		// Update camera state (may reset accumulation if needed)
		ApertureSampler->UpdateCameraState(CameraState);
	}

	// Render amortized samples
	if (!ApertureSampler->IsComplete())
	{
		ApertureSampler->RenderAmortizedSamples();
	}
}

ACineCameraActor* FAccumulationDOFViewportExtension::GetPilotedCineCameraActor() const
{
	if (!LinkedViewportClient)
	{
		return nullptr;
	}

	// Check cinematic lock first (Sequencer), then actor lock (piloted)

	const FLevelViewportActorLock& CinematicLock = LinkedViewportClient->GetCinematicActorLock();

	if (CinematicLock.HasValidLockedActor())
	{
		return Cast<ACineCameraActor>(CinematicLock.GetLockedActor());
	}

	const FLevelViewportActorLock& ActorLock = LinkedViewportClient->GetActorLock();

	return Cast<ACineCameraActor>(ActorLock.GetLockedActor());
}

UCineCameraComponent* FAccumulationDOFViewportExtension::GetPilotedCineCameraComponent() const
{
	if (ACineCameraActor* CineActor = GetPilotedCineCameraActor())
	{
		return CineActor->GetCineCameraComponent();
	}

	return nullptr;
}

UAccumulationDOFComponent* FAccumulationDOFViewportExtension::GetPilotedDOFComponent() const
{
	if (ACineCameraActor* CineActor = GetPilotedCineCameraActor())
	{
		return CineActor->FindComponentByClass<UAccumulationDOFComponent>();
	}

	return nullptr;
}

bool FAccumulationDOFViewportExtension::IsViewModeCompatible(const FSceneView& View) const
{
	EViewModeIndex ViewMode = FindViewMode(View.Family->EngineShowFlags);

	switch (ViewMode)
	{
		case VMI_Lit:
		case VMI_Lit_DetailLighting:
		case VMI_LightingOnly:
		case VMI_ReflectionOverride:

			return true;

		default:
			return false;
	}
}

AccumulationDOF::FApertureSamplerCameraState FAccumulationDOFViewportExtension::BuildCameraStateFromViewport() const
{
	AccumulationDOF::FApertureSamplerCameraState State;

	if (!LinkedViewportClient)
	{
		return State;
	}

	// Check if piloting a CineCameraActor
	ACineCameraActor* CineActor = GetPilotedCineCameraActor();

	if (CineActor)
	{
		State.CineCameraComponent = CineActor->GetCineCameraComponent();
		State.CameraLocation      = CineActor->GetCineCameraComponent()->GetComponentLocation();
		State.CameraRotation      = CineActor->GetCineCameraComponent()->GetComponentRotation();
		State.bInitialized        = true;
	}

	return State;
}

UWorld* FAccumulationDOFViewportExtension::GetWorld() const
{
	if (bIsInvalidated || !LinkedViewportClient)
	{
		return nullptr;
	}

	return LinkedViewportClient->GetWorld();
}

void FAccumulationDOFViewportExtension::RestartAccumulation()
{
	bIsFrozen = false;

	if (ApertureSampler)
	{
		ApertureSampler->Shutdown();
		ApertureSampler = nullptr;
	}
}

void FAccumulationDOFViewportExtension::Unfreeze()
{
	if (bIsFrozen)
	{
		RestartAccumulation();
	}
}

void FAccumulationDOFViewportExtension::CaptureOneshot()
{
	UWorld* World = GetWorld();

	if (!World)
	{
		return;
	}

	AccumulationDOF::FApertureSamplerCameraState CameraState = BuildCameraStateFromViewport();

	if (!CameraState.bInitialized)
	{
		return;
	}

	// Query the viewport for the render resolution. 
	// The aspect-ratio and overscan transforms below run in either case, so we just need
	// the raw viewport size.

	if (!LinkedViewportClient || !LinkedViewportClient->Viewport)
	{
		return;
	}

	FIntPoint RenderResolution = LinkedViewportClient->Viewport->GetSizeXY();

	if (RenderResolution.X <= 0 || RenderResolution.Y <= 0)
	{
		return;
	}

	UCineCameraComponent* CineCam = GetPilotedCineCameraComponent();

	// Adjust for camera's actual filmback aspect ratio when constrained.
	if (CineCam && CineCam->bConstrainAspectRatio)
	{
		const float CameraAspect = GetCameraFilmbackAspectRatio(CineCam);
		if (CameraAspect > 0.0f)
		{
			RenderResolution.X = FMath::RoundToInt(RenderResolution.Y * CameraAspect);
		}
	}

	// Apply overscan scaling

	float OverscanFraction = 0.0f;

	if (CineCam)
	{
		OverscanFraction = CineCam->Overscan;
		if (OverscanFraction > 0.0f)
		{
			const float OverscanScale = 1.0f + OverscanFraction;
			RenderResolution.X = FMath::CeilToInt(RenderResolution.X * OverscanScale);
			RenderResolution.Y = FMath::CeilToInt(RenderResolution.Y * OverscanScale);
		}
	}

	// Build config with OneShot mode

	AccumulationDOF::FApertureSamplerConfig Config;

	Config.World               = World;
	Config.Resolution          = RenderResolution;
	Config.NumSamples          = Settings.NumApertureSamples;
	Config.DOFSplatSize        = Settings.DOFSplatSize;
	Config.Mode                = AccumulationDOF::EApertureSamplerMode::OneShot;
	Config.TemporalHistoryMode = ETemporalHistoryMode::AllSamplesUpdate; // @todo fix temporal reprojection issue where object velocity shifts it on first sample
	Config.OverscanFraction    = OverscanFraction;

	// Query screen percentage (max of CVar minimum and viewport)
	const float MinScreenPercentage = static_cast<float>(CVarAccumulationDOFPreviewScreenPercentageMin.GetValueOnGameThread()) / 100.0f;
	float ViewportScreenPercentage = 1.0f;

	if (FEditorViewportClient* EditorClient = LinkedViewportClient)
	{
		ViewportScreenPercentage = EditorClient->IsPreviewingScreenPercentage()
			? EditorClient->GetPreviewScreenPercentage() / 100.0f
			: EditorClient->GetDefaultPrimaryResolutionFractionTarget();
	}

	Config.ScreenPercentage = FMath::Max(MinScreenPercentage, ViewportScreenPercentage);

	// Use component settings when available
	if (UAccumulationDOFComponent* DOFComp = GetPilotedDOFComponent())
	{
		// Use camera component's NumSamples and DOFSplatSize when bUseCameraSettings is enabled
		if (Settings.bUseCameraSettings)
		{
			Config.NumSamples   = DOFComp->NumSamples;
			Config.DOFSplatSize = DOFComp->DOFSplatSize;
		}

		Config.SamplingPattern                     = DOFComp->SamplingPattern;
		Config.AxialChromaticAberrationIntensity   = DOFComp->AxialChromaticAberrationIntensity;
		Config.AxialChromaticAberrationNumBands    = DOFComp->AxialChromaticAberrationNumBands;
		Config.bSpectralLateralChromaticAberration = DOFComp->bSpectralLateralChromaticAberration;
		Config.SphericalAberration                 = DOFComp->SphericalAberration;
		Config.ComaAberration                      = DOFComp->ComaAberration;
		Config.BokehTexture                        = DOFComp->BokehTexture;
		Config.bEnableBokehTexture                 = DOFComp->bEnableBokehTexture;
		Config.WeightChannel                       = DOFComp->WeightChannel;
		Config.TintStrength                        = DOFComp->TintStrength;
		Config.BokehEdgeSoftness                   = DOFComp->BokehEdgeSoftness;
	}

	Config.ViewModeIndex = CachedViewMode;

	// Initialize and render
	if (!ApertureSampler)
	{
		ApertureSampler = NewObject<UApertureSampler>(GetTransientPackage());
	}
	else
	{
		ApertureSampler->Shutdown();
	}

	if (!ApertureSampler->Initialize(Config, CameraState))
	{
		return;
	}

	const bool bSuccess = ApertureSampler->RenderAllSamples();

	if (!bSuccess)
	{
		// User cancelled or error - don't freeze or cache
		return;
	}

	// Update cached resolution and world
	CachedViewportResolution = RenderResolution;
	CachedWorld = World;
	CachedOverscanFraction = (CineCam && CineCam->bCropOverscan) ? OverscanFraction : 0.0f;

	// Cache result for injection
	CachedPreviewTexture   = ApertureSampler->GetAccumulatedResult();
	CachedProgressFraction = 1.0f;
	bCachedIsComplete      = true;

	if (UTextureRenderTarget2D* PreviewRT = CachedPreviewTexture.Get())
	{
		if (FTextureRenderTargetResource* Resource = PreviewRT->GameThread_GetRenderTargetResource())
		{
			CachedPreviewRHITexture_RenderThread = Resource->GetRenderTargetTexture();
		}
		else
		{
			CachedPreviewRHITexture_RenderThread = nullptr;
		}
	}
	else
	{
		CachedPreviewRHITexture_RenderThread = nullptr;
	}

	// Freeze to preserve one-shot result from scene motion resets
	bIsFrozen = true;
	FrozenCameraParams = CaptureCameraParams();
}

void FAccumulationDOFViewportExtension::InvalidateViewportClient()
{
	// Set flag first to guard against callbacks firing during de-registration
	bIsInvalidated = true;

	// De-register delegates before nulling the pointer to prevent callbacks from firing
	if (GEngine)
	{
		GEngine->OnActorMoved().Remove(OnActorMovedHandle);
		GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedHandle);
		GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedHandle);
		GEngine->OnComponentTransformChanged().Remove(OnComponentTransformChangedHandle);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);

	// Clear handles

	OnActorMovedHandle.Reset();
	OnLevelActorAddedHandle.Reset();
	OnLevelActorDeletedHandle.Reset();
	OnComponentTransformChangedHandle.Reset();
	OnObjectPropertyChangedHandle.Reset();

	LinkedViewportClient = nullptr;
}

float FAccumulationDOFViewportExtension::GetProgressFraction() const
{
	if (ApertureSampler)
	{
		return ApertureSampler->GetProgress().GetProgressFraction();
	}

	return 0.0f;
}

bool FAccumulationDOFViewportExtension::IsComplete() const
{
	return ApertureSampler && ApertureSampler->IsComplete();
}

FScreenPassTexture FAccumulationDOFViewportExtension::ProcessAtMotionBlurPass_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FPostProcessMaterialInputs& InOutInputs)
{
	const FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));

	if (!SceneColor.IsValid())
	{
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, SceneColor, View.GetOverwriteLoadAction(), TEXT("AccumulationDOFOutput"));
	}

	// Only inject if we have valid accumulated data

	if (!CachedPreviewRHITexture_RenderThread)
	{
		AddCopyTexturePass(GraphBuilder, SceneColor.Texture, Output.Texture);
		return MoveTemp(Output);
	}

	// Import accumulated texture to RDG
	FRDGTextureRef AccumulatedRDG = GraphBuilder.RegisterExternalTexture(
		CreateRenderTarget(
			CachedPreviewRHITexture_RenderThread,
			TEXT("AccumulatedDOF"))
	);

	// Use cached progress state

	const float Progress = CachedProgressFraction;
	const bool bDrawProgressBar = !bCachedIsComplete;

	// Use SceneColor's ViewRect
	FIntRect InjectionViewRect = SceneColor.ViewRect;

	const bool bDrawPreviewLabel = CVarAccumulationDOFShowPreviewLabel.GetValueOnRenderThread() > 0;

	// LensDistortionLUT: only apply in our shader when the engine has routed the LUT to
	// the TSR pass (i.e. it has disabled PrimaryUpscale's LUT path). When PassLocation
	// is PrimaryUpscale, the engine's spatial upscale will apply the LUT itself and
	// applying here would double-distort.
	const FLensDistortionLUT& LensDistortionLUT = LensDistortion::GetLUTUnsafe(View);
	const FLensDistortionLUT* const LUTToApplyInShader =
		(LensDistortion::GetPassLocationUnsafe(View) == LensDistortion::EPassLocation::TSR)
			? &LensDistortionLUT
			: nullptr;

	AccumulationDOF::InjectToSceneColor(
		GraphBuilder,
		AccumulatedRDG,
		Output.Texture,
		InjectionViewRect,
		Progress,
		bDrawProgressBar,
		CachedOverscanFraction,
		bDrawPreviewLabel,
		bCachedIsFrozen,
		/*bApplyAspectFit=*/true,
		LUTToApplyInShader);

	return MoveTemp(Output);
}

FCameraParamsSnapshot FAccumulationDOFViewportExtension::CaptureCameraParams() const
{
	FCameraParamsSnapshot Params;

	if (ACineCameraActor* CineActor = GetPilotedCineCameraActor())
	{

		if (UCineCameraComponent* CineCam = CineActor->GetCineCameraComponent())
		{
			Params.Location = CineCam->GetComponentLocation();
			Params.Rotation = CineCam->GetComponentRotation();
			Params.FocusDistance = CineCam->FocusSettings.ManualFocusDistance;
			Params.Aperture = CineCam->CurrentAperture;
			Params.FocalLength = CineCam->CurrentFocalLength;
		}
	}

	return Params;
}

void FAccumulationDOFViewportExtension::CheckFrozenCameraStateChanged()
{
	if (bIsFrozen && !CaptureCameraParams().Equals(FrozenCameraParams))
	{
		Unfreeze();
	}
}

bool FAccumulationDOFViewportExtension::IsActorRelevantToOurWorld(AActor* Actor) const
{
	if (!Actor)
	{
		return false;
	}

	UWorld* OurWorld = GetWorld();

	return OurWorld && (Actor->GetWorld() == OurWorld);
}

void FAccumulationDOFViewportExtension::OnSceneActorMoved(AActor* Actor)
{
	if (bIsInvalidated)
	{
		return;
	}

	if (IsActorRelevantToOurWorld(Actor) && !Actor->IsHiddenEd())
	{
		IncrementInvalidationCounter();
	}
}

void FAccumulationDOFViewportExtension::OnSceneLevelActorAdded(AActor* Actor)
{
	if (bIsInvalidated)
	{
		return;
	}

	if (IsActorRelevantToOurWorld(Actor) && !Actor->IsHiddenEd())
	{
		IncrementInvalidationCounter();
	}
}

void FAccumulationDOFViewportExtension::OnSceneLevelActorDeleted(AActor* Actor)
{
	if (bIsInvalidated)
	{
		return;
	}

	if (IsActorRelevantToOurWorld(Actor) && !Actor->IsHiddenEd())
	{
		IncrementInvalidationCounter();
	}
}

void FAccumulationDOFViewportExtension::OnSceneComponentTransformChanged(
	USceneComponent* Component, ETeleportType Teleport)
{
	if (bIsInvalidated)
	{
		return;
	}

	if (Component)
	{
		AActor* Owner = Component->GetOwner();

		if (IsActorRelevantToOurWorld(Owner) && !Owner->IsHiddenEd())
		{
			IncrementInvalidationCounter();
		}
	}
}

void FAccumulationDOFViewportExtension::OnSceneObjectPropertyChanged(
	UObject* Object,
	FPropertyChangedEvent& Event
)
{
	if (bIsInvalidated)
	{
		return;
	}

	AActor* Actor = Cast<AActor>(Object);

	if (!Actor)
	{
		UActorComponent* Component = Cast<UActorComponent>(Object);
		if (Component)
		{
			Actor = Component->GetOwner();
		}
	}

	if (!Actor || !IsActorRelevantToOurWorld(Actor))
	{
		return;
	}

	// Always invalidate on visibility changes (hiding or showing affects the scene)
	const FName PropertyName = Event.GetMemberPropertyName();
	const bool bIsVisibilityChange = (PropertyName == GET_MEMBER_NAME_CHECKED(AActor, bHiddenEdLevel)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(AActor, bHiddenEd)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(AActor, bHiddenEdLayer));

	if (bIsVisibilityChange || !Actor->IsHiddenEd())
	{
		IncrementInvalidationCounter();
	}
}

void FAccumulationDOFViewportExtension::IncrementInvalidationCounter()
{
	SceneInvalidationCounter++;
	bDOFSettingsNeedReinit = true;
}
