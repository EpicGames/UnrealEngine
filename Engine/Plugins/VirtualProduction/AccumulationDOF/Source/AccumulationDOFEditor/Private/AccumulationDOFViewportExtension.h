// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelEditorViewport.h"
#include "SceneViewExtension.h"
#include "UObject/GCObject.h"

#include "AccumulationDOFViewportSettings.h"

class FViewportClient;
class UApertureSampler;
class UCineCameraComponent;
class UWorld;
class UAccumulationDOFComponent;
class ACineCameraActor;
class UTextureRenderTarget2D;

namespace AccumulationDOF
{
	struct FApertureSamplerCameraState;
}

/** Snapshot of camera parameters for change detection */
struct FCameraParamsSnapshot
{
	FVector Location = FVector::ZeroVector;
	FRotator Rotation = FRotator::ZeroRotator;
	float FocusDistance = 0.0f;
	float Aperture = 0.0f;
	float FocalLength = 0.0f;

	bool Equals(const FCameraParamsSnapshot& Other, float Tolerance = 0.01f) const
	{
		return Location.Equals(Other.Location, Tolerance)
			&& Rotation.Equals(Other.Rotation, Tolerance)
			&& FMath::IsNearlyEqual(FocusDistance, Other.FocusDistance, Tolerance)
			&& FMath::IsNearlyEqual(Aperture, Other.Aperture, 0.001f)
			&& FMath::IsNearlyEqual(FocalLength, Other.FocalLength, Tolerance);
	}
};

/**
 * Scene View Extension for viewport-based Accumulation DOF preview.
 * Orchestrates amortized aperture sample rendering for a specific viewport.
 */
class FAccumulationDOFViewportExtension
	: public FSceneViewExtensionBase
	, public FGCObject
{
public:
	FAccumulationDOFViewportExtension(const FAutoRegister& AutoRegister, FLevelEditorViewportClient* AssociatedViewportClient);
	virtual ~FAccumulationDOFViewportExtension();

	//~ Begin ISceneViewExtension interface

	virtual int32 GetPriority() const override
	{
		return -5;
	}

	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

	virtual void SubscribeToPostProcessingPass(
		EPostProcessingPass Pass,
		const FSceneView& View,
		FAfterPassCallbackDelegateArray& InOutPassCallbacks,
		bool bIsPassEnabled) override;

	//~ End ISceneViewExtension interface

	//~ Begin FGCObject interface

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual FString GetReferencerName() const override
	{
		return TEXT("FAccumulationDOFViewportExtension");
	}

	//~ End FGCObject interface

	/** Get the associated viewport client */
	FViewportClient* GetAssociatedViewportClient() const
	{
		return LinkedViewportClient;
	}

	/** Invalidate the viewport client pointer (call before removing from manager) */
	void InvalidateViewportClient();

	/** Get settings */
	FAccumulationDOFViewportSettings& GetSettings()
	{
		return Settings;
	}

	/** Get settings (const) */
	const FAccumulationDOFViewportSettings& GetSettings() const
	{
		return Settings;
	}

	/** Force restart of accumulation */
	void RestartAccumulation();

	/** Perform blocking oneshot capture of all samples */
	void CaptureOneshot();

	/** Get progress fraction (0-1) */
	float GetProgressFraction() const;

	/** Check if accumulation is complete */
	bool IsComplete() const;

	/** Check if preview is frozen (one-shot result preserved from scene changes) */
	bool IsFrozen() const { return bIsFrozen; }

	/** Unfreeze and restart accumulation */
	void Unfreeze();

private:

	/** Tick amortized rendering (called once per frame) */
	void TickAmortizedRendering();

	/** Get the CineCameraActor being piloted */
	ACineCameraActor* GetPilotedCineCameraActor() const;

	/** Get CineCameraComponent from piloted camera */
	UCineCameraComponent* GetPilotedCineCameraComponent() const;

	/** Get AccumulationDOFComponent from piloted camera */
	UAccumulationDOFComponent* GetPilotedDOFComponent() const;

	/** Check if current view mode is compatible with DOF rendering */
	bool IsViewModeCompatible(const FSceneView& View) const;

	/** Build camera state from viewport. Only valid when piloting a CineCameraActor. */
	AccumulationDOF::FApertureSamplerCameraState BuildCameraStateFromViewport() const;

	/** Get world from viewport */
	UWorld* GetWorld() const;

	/** Inject callback for post-processing */
	FScreenPassTexture ProcessAtMotionBlurPass_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FPostProcessMaterialInputs& Inputs);

	/** Callback when an actor is moved */
	void OnSceneActorMoved(AActor* Actor);

	/** Callback when a level actor is added */
	void OnSceneLevelActorAdded(AActor* Actor);

	/** Callback when a level actor is deleted */
	void OnSceneLevelActorDeleted(AActor* Actor);

	/** Callback when a component transform changes */
	void OnSceneComponentTransformChanged(USceneComponent* Component, ETeleportType Teleport);

	/** Callback when an object property changes */
	void OnSceneObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& Event);

	/** Increment the scene invalidation counter */
	void IncrementInvalidationCounter();

	/** Check if an actor is relevant to our world */
	bool IsActorRelevantToOurWorld(AActor* Actor) const;

	/** Check if camera state changed while frozen, unfreeze if so */
	void CheckFrozenCameraStateChanged();

	/** Capture current camera parameters for change detection */
	FCameraParamsSnapshot CaptureCameraParams() const;

private:

	/** The viewport client this extension is bound to */
	FLevelEditorViewportClient* LinkedViewportClient = nullptr;

	/** Flag set when viewport is invalidated to guard against dangling pointer access */
	bool bIsInvalidated = false;

	/** Per-viewport settings */
	FAccumulationDOFViewportSettings Settings;

	/** Aperture sampler instance (owns render targets) */
	TObjectPtr<UApertureSampler> ApertureSampler = nullptr;

	/** Cached world reference */
	TWeakObjectPtr<UWorld> CachedWorld;

	/** Frame counter for single-tick-per-frame detection */
	uint64 LastTickFrame = 0;

	/** Cached viewport resolution for change detection */
	FIntPoint CachedViewportResolution = FIntPoint::ZeroValue;

	/** Cached view rect from current frame (set in SetupView) */
	FIntRect CachedViewRect;

	/** Whether we're currently in a compatible view mode */
	bool bIsViewModeCompatible = false;

	/** Cached view mode for propagating to aperture sampler config */
	EViewModeIndex CachedViewMode = VMI_Lit;

	/** Cached preview texture prepared on game thread for render thread injection */
	TWeakObjectPtr<UTextureRenderTarget2D> CachedPreviewTexture;

	/** Cached RHI texture for render thread access */
	FTextureRHIRef CachedPreviewRHITexture_RenderThread;

	/** Cached progress fraction for render thread (avoids race condition) */
	float CachedProgressFraction = 0.0f;

	/** Cached completion state for render thread (avoids race condition) */
	bool bCachedIsComplete = false;

	/** Cached frozen state for render thread */
	bool bCachedIsFrozen = false;

	/** Cached overscan fraction for render thread (only set when bCropOverscan is true) */
	float CachedOverscanFraction = 0.0f;

	/** Delegate handle for actor moved events */
	FDelegateHandle OnActorMovedHandle;

	/** Delegate handle for level actor added events */
	FDelegateHandle OnLevelActorAddedHandle;

	/** Delegate handle for level actor deleted events */
	FDelegateHandle OnLevelActorDeletedHandle;

	/** Delegate handle for component transform changed events */
	FDelegateHandle OnComponentTransformChangedHandle;

	/** Delegate handle for object property changed events */
	FDelegateHandle OnObjectPropertyChangedHandle;

	/** Counter incremented when scene changes are detected */
	uint32 SceneInvalidationCounter = 0;

	/** Last seen invalidation counter for change detection */
	uint32 LastSeenInvalidationCounter = 0;

	/** Set when DOF component settings change, forcing full reinit to pick up new config */
	bool bDOFSettingsNeedReinit = false;

	/** Frozen state: prevents scene-change resets after one-shot capture */
	bool bIsFrozen = false;

	/** Cached camera state for detecting Sequencer-driven changes while frozen */
	FCameraParamsSnapshot FrozenCameraParams;
};
