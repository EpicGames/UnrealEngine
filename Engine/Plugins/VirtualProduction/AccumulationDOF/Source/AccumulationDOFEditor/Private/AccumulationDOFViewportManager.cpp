// Copyright Epic Games, Inc. All Rights Reserved.

#include "AccumulationDOFViewportManager.h"

#include "AccumulationDOFViewportExtension.h"
#include "AccumulationDOFViewportSettings.h"

#include "LevelEditorViewport.h"
#include "SceneViewExtension.h"


FAccumulationDOFViewportManager::~FAccumulationDOFViewportManager()
{
	for (TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Extension : ViewportExtensions)
	{
		if (Extension.IsValid())
		{
			Extension->InvalidateViewportClient();
		}
	}

	ViewportExtensions.Empty();
}

FAccumulationDOFViewportSettings& FAccumulationDOFViewportManager::FindOrAddViewportSettings(FLevelEditorViewportClient* InViewportClient)
{
	// Find existing extension

	TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Found)
	{
		return (*Found)->GetSettings();
	}

	// Create new extension

	TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe> NewExtension =
		FSceneViewExtensions::NewExtension<FAccumulationDOFViewportExtension>(InViewportClient);

	ViewportExtensions.Add(NewExtension);

	return NewExtension->GetSettings();
}

const FAccumulationDOFViewportSettings* FAccumulationDOFViewportManager::GetViewportSettings(const FViewportClient* InViewportClient) const
{
	const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Found)
	{
		return &(*Found)->GetSettings();
	}

	return nullptr;
}

bool FAccumulationDOFViewportManager::RemoveViewportSettings(const FViewportClient* InViewportClient)
{
	int32 Index = ViewportExtensions.IndexOfByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Index != INDEX_NONE)
	{
		ViewportExtensions[Index]->InvalidateViewportClient();
		ViewportExtensions.RemoveAtSwap(Index);

		return true;
	}

	return false;
}

bool FAccumulationDOFViewportManager::IsTrackingViewport(const FViewportClient* InViewportClient) const
{
	return ViewportExtensions.ContainsByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});
}

void FAccumulationDOFViewportManager::RestartAccumulation(FViewportClient* InViewportClient)
{
	TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Found && Found->IsValid())
	{
		(*Found)->RestartAccumulation();
	}
}

void FAccumulationDOFViewportManager::CaptureOneshot(FViewportClient* InViewportClient)
{
	TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Found && Found->IsValid())
	{
		(*Found)->CaptureOneshot();
	}
}

void FAccumulationDOFViewportManager::Unfreeze(FViewportClient* InViewportClient)
{
	TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	if (Found && Found->IsValid())
	{
		(*Found)->Unfreeze();
	}
}

TSharedPtr<FAccumulationDOFViewportExtension> FAccumulationDOFViewportManager::GetViewportExtension(const FViewportClient* InViewportClient) const
{
	const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>* Found = ViewportExtensions.FindByPredicate(
		[InViewportClient](const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Other)
		{
			return Other.IsValid() && InViewportClient == Other->GetAssociatedViewportClient();
		});

	return Found ? *Found : nullptr;
}

void FAccumulationDOFViewportManager::RemoveInvalidViewports(const TArray<FLevelEditorViewportClient*>& ValidViewports)
{
	for (int32 SveIdx = ViewportExtensions.Num() - 1; SveIdx >= 0; --SveIdx)
	{
		const TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>& Extension = ViewportExtensions[SveIdx];

		if (!Extension.IsValid())
		{
			ViewportExtensions.RemoveAtSwap(SveIdx);
			continue;
		}

		const FViewportClient* ExtensionViewport = Extension->GetAssociatedViewportClient();

		const bool bIsValid = ValidViewports.ContainsByPredicate(
			[ExtensionViewport](const FLevelEditorViewportClient* Other)
			{
				return Other == ExtensionViewport;
			});

		if (!bIsValid)
		{
			Extension->InvalidateViewportClient();
			ViewportExtensions.RemoveAtSwap(SveIdx);
		}
	}
}
