// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"


class FAccumulationDOFViewportExtension;
class FLevelEditorViewportClient;
class FViewportClient;

struct FAccumulationDOFViewportSettings;


/**
 * Manages per-viewport Accumulation DOF configurations and scene view extensions.
 */
class FAccumulationDOFViewportManager : public TSharedFromThis<FAccumulationDOFViewportManager>
{
public:
	FAccumulationDOFViewportManager() = default;
	~FAccumulationDOFViewportManager();

	/**
	 * Find or create configuration for a viewport.
	 * Creates a scene view extension if one doesn't exist.
	 */
	FAccumulationDOFViewportSettings& FindOrAddViewportSettings(FLevelEditorViewportClient* InViewportClient);

	/** Get settings for a viewport if it's tracked */
	const FAccumulationDOFViewportSettings* GetViewportSettings(const FViewportClient* InViewportClient) const;

	/** Remove viewport tracking and cleanup */
	bool RemoveViewportSettings(const FViewportClient* InViewportClient);

	/** Remove all extensions for viewports not in the valid list */
	void RemoveInvalidViewports(const TArray<FLevelEditorViewportClient*>& ValidViewports);

	/** Check if viewport is being tracked */
	bool IsTrackingViewport(const FViewportClient* InViewportClient) const;

	/** Force a viewport to restart its accumulation */
	void RestartAccumulation(FViewportClient* InViewportClient);

	/** Perform blocking oneshot capture for a viewport */
	void CaptureOneshot(FViewportClient* InViewportClient);

	/** Unfreeze a viewport (clears frozen state and restarts accumulation) */
	void Unfreeze(FViewportClient* InViewportClient);

	/** Get the extension for a viewport if it exists */
	TSharedPtr<FAccumulationDOFViewportExtension> GetViewportExtension(const FViewportClient* InViewportClient) const;

private:

	/** SVEs for each tracked viewport */
	TArray<TSharedPtr<FAccumulationDOFViewportExtension, ESPMode::ThreadSafe>> ViewportExtensions;
};
