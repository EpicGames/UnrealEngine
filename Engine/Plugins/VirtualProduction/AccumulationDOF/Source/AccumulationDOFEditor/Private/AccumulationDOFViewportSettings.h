// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Per-viewport settings for Accumulation DOF preview.
 */
struct FAccumulationDOFViewportSettings
{
	/** Accumulation DOF is enabled for this viewport */
	bool bIsEnabled = false;

	/** Use settings from camera's AccumulationDOFComponent instead of overrides below */
	bool bUseCameraSettings = false;

	/** Number of aperture samples to accumulate */
	int32 NumApertureSamples = 256;

	/** Fraction of main aperture diameter for DOF splat size. Expected range 0-1, clamped by UI. */
	float DOFSplatSize = 0.125f;

	/** Number of samples to render per frame in amortized mode. */
	int32 SamplesPerFrame = 2;
};

/** Delegate for settings changes */
DECLARE_DELEGATE_OneParam(FOnAccumulationDOFSettingsChanged, const FAccumulationDOFViewportSettings&);
